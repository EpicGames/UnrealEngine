// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimedDataMonitorCalibration.h"
#include "TimedDataMonitorSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "Misc/App.h"

/** The algo for Calibration and TimeCorrection will use those data for their examples and comments. */
// EvaluationTime == 50.
//A1 10  11
//A2                   48  49  50  51
//A3     11  12
//B1                                                 99  100
//B2                                                     100
//B3                                                     100  101
//C1 10  11
//C2                                                     100
//D1                       49  50  51
//D2                   48  49  50  51

namespace TimedDataCalibration
{
	struct FChannelSampleMinMax
	{
		FTimedDataMonitorChannelIdentifier ChannelIdentifier;
		double MinAsSeconds;
		double MaxAsSeconds;
	};

	struct FSmallestBiggestSample
	{
		double SmallestMinInSeconds = TNumericLimits<double>::Max();
		double BiggestMaxInSeconds = TNumericLimits<double>::Lowest();
		double BiggerMinInSeconds = TNumericLimits<double>::Lowest();
		double SmallestMaxInSeconds = TNumericLimits<double>::Max();
	};

	struct FEnabledInput
	{
		FTimedDataMonitorInputIdentifier InputIdentifier;
		TArray<FTimedDataMonitorChannelIdentifier> ChannelIdentifiers;
	};

	struct FChannelBufferSize
	{
		FTimedDataMonitorChannelIdentifier ChannelIdentifier;
		int32 BufferSize;
	};
	
	struct FInputBufferSize
	{
		FTimedDataMonitorInputIdentifier InputIdentifier;
		int32 BufferSize;
	};


	TArray<FTimedDataMonitorChannelIdentifier> GetEnabledChannelIdentifiers(UTimedDataMonitorSubsystem* TimedDataMonitor, const FTimedDataMonitorInputIdentifier& InputIdentifier)
	{
		TArray<FTimedDataMonitorChannelIdentifier> AllChannelIdentifier = TimedDataMonitor->GetInputChannels(InputIdentifier);

		// Remove channel that are not enabled
		for (int32 Index = AllChannelIdentifier.Num() - 1; Index >= 0; --Index)
		{
			if (!TimedDataMonitor->IsChannelEnabled(AllChannelIdentifier[Index]))
			{
				AllChannelIdentifier.RemoveAtSwap(Index);
			}
		}
		return AllChannelIdentifier;
	}


	TArray<FEnabledInput> GetEnabledInputs(UTimedDataMonitorSubsystem* TimedDataMonitor)
	{
		TArray<FTimedDataMonitorInputIdentifier> Inputs = TimedDataMonitor->GetAllInputs();
		TArray<FEnabledInput> AllValidInputIndentifiers;
		AllValidInputIndentifiers.Reset(Inputs.Num());

		for (const FTimedDataMonitorInputIdentifier& InputIdentifier : Inputs)
		{
			FEnabledInput NewInput;
			NewInput.InputIdentifier = InputIdentifier;
			NewInput.ChannelIdentifiers = GetEnabledChannelIdentifiers(TimedDataMonitor, InputIdentifier);
			if (NewInput.ChannelIdentifiers.Num() > 0)
			{
				AllValidInputIndentifiers.Add(MoveTemp(NewInput));
			}
		}

		return AllValidInputIndentifiers;
	}


	TArray<FChannelSampleMinMax> GetChannelsMinMax(UTimedDataMonitorSubsystem* TimedDataMonitor, ETimedDataInputEvaluationType EvaluationType, const TArray<FTimedDataMonitorChannelIdentifier>& Channels, float EvaluationOffsetInSeconds, int32 NumberOfSTD)
	{
		TArray<FChannelSampleMinMax> Result;
		Result.Reset(Channels.Num());
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Channels)
		{
			check(TimedDataMonitor->GetChannelNumberOfSamples(ChannelIdentifier) > 0);
			FChannelSampleMinMax NewItem;
			NewItem.ChannelIdentifier = ChannelIdentifier;
			NewItem.MinAsSeconds = TimedDataMonitor->GetChannelOldestDataTime(ChannelIdentifier).AsSeconds(EvaluationType) + EvaluationOffsetInSeconds;
			NewItem.MaxAsSeconds = TimedDataMonitor->GetChannelNewestDataTime(ChannelIdentifier).AsSeconds(EvaluationType) + EvaluationOffsetInSeconds;
			if (NumberOfSTD > 0)
			{
				NewItem.MinAsSeconds += TimedDataMonitor->GetChannelEvaluationDistanceToOldestSampleStandardDeviation(ChannelIdentifier) * NumberOfSTD;
				NewItem.MaxAsSeconds -= TimedDataMonitor->GetChannelEvaluationDistanceToNewestSampleStandardDeviation(ChannelIdentifier) * NumberOfSTD;
			}
			Result.Add(NewItem);
		}
		return Result;
	}


	FSmallestBiggestSample GetSmallestBiggestSample(const TArray<FChannelSampleMinMax>& Channels)
	{
		FSmallestBiggestSample Result;
		for (const FChannelSampleMinMax& ChannelItt : Channels)
		{
			Result.SmallestMinInSeconds = FMath::Min(ChannelItt.MinAsSeconds, Result.SmallestMinInSeconds);	//A == 10, B == 99, C == 10, D == 48
			Result.BiggestMaxInSeconds = FMath::Max(ChannelItt.MaxAsSeconds, Result.BiggestMaxInSeconds);	//A == 51, B == 101, C == 100, D == 51

			Result.BiggerMinInSeconds = FMath::Max(ChannelItt.MinAsSeconds, Result.BiggerMinInSeconds);		//A == 48, B == 100, C == 10, D == 49
			Result.SmallestMaxInSeconds = FMath::Min(ChannelItt.MaxAsSeconds, Result.SmallestMaxInSeconds);	//A == 11, B == 100, C == 11, D == 51
		}
		return Result;
	}


	bool IsInRange(const FChannelSampleMinMax& SampleMinMax, double InSeconds)
	{
		return InSeconds >= SampleMinMax.MinAsSeconds
			&& InSeconds <= SampleMinMax.MaxAsSeconds;
	}


	bool IsInRange(const TArray<FChannelSampleMinMax>& ChannelSamplesMinMax, double InSeconds)
	{
		if (ChannelSamplesMinMax.Num() == 0)
		{
			return false;
		}
		for (const TimedDataCalibration::FChannelSampleMinMax& SampleMinMax : ChannelSamplesMinMax)
		{
			if (!IsInRange(SampleMinMax, InSeconds))
			{
				return false;
			}
		}
		return true;
	}


	double CalculateAverageInDeltaTimeBetweenSample(ETimedDataInputEvaluationType EvaluationType, const TArray<FTimedDataChannelSampleTime>& SampleTimes)
	{
		double Average = 0.0;
		if (SampleTimes.Num() >= 2)
		{
			// Get the average of the last 10 samples in seconds
			const int32 AvgCounter = FMath::Min(SampleTimes.Num() - 1, 10 - 1);

			const int32 SampleTimeNum = SampleTimes.Num();

			for (int32 Index = 1; Index <= AvgCounter; ++Index)
			{
				double Delta = SampleTimes[SampleTimeNum - Index].AsSeconds(EvaluationType) - SampleTimes[SampleTimeNum - Index - 1].AsSeconds(EvaluationType);
				Average += (Delta - Average) / (double)Index;
			}
		}
		else
		{
			Average = FApp::GetDeltaTime(); // was not able to find a correct delta time. guess one.
		}
		return Average;
	}


	double CalculateAverageInDeltaTimeBetweenSample(UTimedDataMonitorSubsystem* TimedDataMonitor, ETimedDataInputEvaluationType EvaluationType, const TArray<FTimedDataMonitorChannelIdentifier>& ChannelIdentifiers)
	{
		double AverageBetweenSample = 0.0;
		int32 AverageCounter = 0;
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : ChannelIdentifiers)
		{
			TArray<FTimedDataChannelSampleTime> AllSamplesTimes = TimedDataMonitor->GetTimedDataChannel(ChannelIdentifier)->GetDataTimes();
			if (AllSamplesTimes.Num() > 1)
			{
				const double CurrentAverageBetweenSample = CalculateAverageInDeltaTimeBetweenSample(EvaluationType, AllSamplesTimes);

				++AverageCounter;
				AverageBetweenSample += (CurrentAverageBetweenSample - AverageBetweenSample) / (double)AverageCounter;
			}
		}
		if (FMath::IsNearlyZero(AverageBetweenSample))
		{
			AverageBetweenSample = FApp::GetDeltaTime();
		}
		return AverageBetweenSample;
	}


	TArray<FChannelBufferSize> GetNewBufferSize(UTimedDataMonitorSubsystem* TimedDataMonitor, ETimedDataInputEvaluationType EvaluationType, const TArray<FChannelSampleMinMax>& SampleTimes, double CalibrationTargetSeconds, int32 NumberOfSTD)
	{
		TArray<FChannelBufferSize> Results;
		Results.Reserve(SampleTimes.Num());
		for (const FChannelSampleMinMax& SampleMinMax : SampleTimes)
		{
			double TotalSecondsNeeded = SampleMinMax.MaxAsSeconds - CalibrationTargetSeconds;
			check(TotalSecondsNeeded >= 0.0);

			ITimedDataInputChannel* ChannelItem = TimedDataMonitor->GetTimedDataChannel(SampleMinMax.ChannelIdentifier);
			check(ChannelItem);

			TArray<FTimedDataChannelSampleTime> AllSamplesTimes = ChannelItem->GetDataTimes();
			const double AverageSecondsBetweenSample = CalculateAverageInDeltaTimeBetweenSample(EvaluationType, AllSamplesTimes);

			double STDExtraTime = 0.0;
			if (NumberOfSTD != 0)
			{
				STDExtraTime = TimedDataMonitor->GetChannelEvaluationDistanceToOldestSampleMean(SampleMinMax.ChannelIdentifier) * NumberOfSTD
				+ TimedDataMonitor->GetChannelEvaluationDistanceToNewestSampleMean(SampleMinMax.ChannelIdentifier) * NumberOfSTD;
			}

			FChannelBufferSize NewBufferSize;
			NewBufferSize.ChannelIdentifier = SampleMinMax.ChannelIdentifier;
			NewBufferSize.BufferSize = FMath::CeilToInt((TotalSecondsNeeded / AverageSecondsBetweenSample) + AverageSecondsBetweenSample + STDExtraTime); // for media, get a full frame
			Results.Add(NewBufferSize);
		}
		return Results;
	}


	void SplitBufferSizeForDoesControlledByInput(UTimedDataMonitorSubsystem* TimedDataMonitor, const TArray<FChannelBufferSize>& ChannelBuffersSize, TArray<FInputBufferSize>& OutNewInputSize, TArray<FChannelBufferSize>& OutNewChannelSize)
	{
		OutNewInputSize.Reset();
		OutNewChannelSize.Reset();
		for (const FChannelBufferSize& ChannelBufferSize : ChannelBuffersSize)
		{
			FTimedDataMonitorInputIdentifier InputIdentifier = TimedDataMonitor->GetChannelInput(ChannelBufferSize.ChannelIdentifier);
			if (TimedDataMonitor->IsDataBufferSizeControlledByInput(InputIdentifier))
			{
				FInputBufferSize* FoundInputBufferSize = OutNewInputSize.FindByPredicate([&InputIdentifier](const FInputBufferSize& Other) { return Other.InputIdentifier == InputIdentifier; });
				if (FoundInputBufferSize)
				{
					FoundInputBufferSize->BufferSize = FMath::Max(FoundInputBufferSize->BufferSize, ChannelBufferSize.BufferSize);
				}
				else
				{
					FInputBufferSize NewBufferSize;
					NewBufferSize.InputIdentifier = InputIdentifier;
					NewBufferSize.BufferSize = ChannelBufferSize.BufferSize;
					OutNewInputSize.Add(NewBufferSize);
				}
			}
			else
			{
				OutNewChannelSize.Add(ChannelBufferSize);
			}
		}
	}


	enum class EResizeBufferResult
	{
		NotEnoughData,
		BufferCouldNotBeResize,
		Success,
	};
	EResizeBufferResult ResizeBuffers(UTimedDataMonitorSubsystem* TimedDataMonitor, const TArray<FInputBufferSize>& RequestedInputSizes, const TArray<FChannelBufferSize>& RequestedChannelSizes
		, bool bCanShrink, bool bFailedIfBufferCantBeResize
		, TArray<FTimedDataMonitorInputIdentifier>& OutFailedInputIdendifiers, TArray<FTimedDataMonitorChannelIdentifier>& OutFailedChannelIdentifiers)
	{
		EResizeBufferResult Result = EResizeBufferResult::Success;

		// Does the new size is bigger than we they currently have
		TArray<FInputBufferSize> PreviousInputSizes;
		TArray<FChannelBufferSize> PreviousChannelSizes;
		{
			for (const FInputBufferSize& RequestedBufferSize : RequestedInputSizes)
			{
				FInputBufferSize& PreviousBufferSize = PreviousInputSizes.Emplace_GetRef();
				PreviousBufferSize.InputIdentifier = RequestedBufferSize.InputIdentifier;
				PreviousBufferSize.BufferSize = TimedDataMonitor->GetInputDataBufferSize(RequestedBufferSize.InputIdentifier);
				if (!bCanShrink && PreviousBufferSize.BufferSize >= RequestedBufferSize.BufferSize)
				{
					OutFailedInputIdendifiers.Add(RequestedBufferSize.InputIdentifier);
					Result = EResizeBufferResult::NotEnoughData;
				}
			}
			for (const FChannelBufferSize& RequestedBufferSize : RequestedChannelSizes)
			{
				FChannelBufferSize PreviousBufferSize = PreviousChannelSizes.Emplace_GetRef();
				PreviousBufferSize.ChannelIdentifier = RequestedBufferSize.ChannelIdentifier;
				PreviousBufferSize.BufferSize = TimedDataMonitor->GetChannelDataBufferSize(RequestedBufferSize.ChannelIdentifier);
				if (!bCanShrink && PreviousBufferSize.BufferSize >= RequestedBufferSize.BufferSize)
				{
					OutFailedChannelIdentifiers.Add(RequestedBufferSize.ChannelIdentifier);
					Result = EResizeBufferResult::NotEnoughData;
				}
			}
			if (Result == EResizeBufferResult::NotEnoughData)
			{
				return Result;
			}
		}

		// Increase the buffer size.
		{
			for (const TimedDataCalibration::FInputBufferSize& RequestedBufferSize : RequestedInputSizes)
			{
				TimedDataMonitor->SetInputDataBufferSize(RequestedBufferSize.InputIdentifier, RequestedBufferSize.BufferSize);
				if (bFailedIfBufferCantBeResize)
				{
					int32 NewBufferSize = TimedDataMonitor->GetInputDataBufferSize(RequestedBufferSize.InputIdentifier);
					const TimedDataCalibration::FInputBufferSize* PreviousBufferSize = PreviousInputSizes.FindByPredicate([=](const TimedDataCalibration::FInputBufferSize& Other) { return Other.InputIdentifier == RequestedBufferSize.InputIdentifier; });
					check(PreviousBufferSize);
					if (PreviousBufferSize->BufferSize == NewBufferSize)
					{
						OutFailedInputIdendifiers.AddUnique(RequestedBufferSize.InputIdentifier);
						Result = EResizeBufferResult::BufferCouldNotBeResize;
					}
				}
			}
			for (const TimedDataCalibration::FChannelBufferSize& RequestedBufferSize : RequestedChannelSizes)
			{
				TimedDataMonitor->SetChannelDataBufferSize(RequestedBufferSize.ChannelIdentifier, RequestedBufferSize.BufferSize);
				if (bFailedIfBufferCantBeResize)
				{
					int32 NewBufferSize = TimedDataMonitor->GetChannelDataBufferSize(RequestedBufferSize.ChannelIdentifier);
					const TimedDataCalibration::FChannelBufferSize* PreviousBufferSize = PreviousChannelSizes.FindByPredicate([=](const TimedDataCalibration::FChannelBufferSize& Other) { return Other.ChannelIdentifier == RequestedBufferSize.ChannelIdentifier; });
					check(PreviousBufferSize);
					if (PreviousBufferSize->BufferSize == NewBufferSize)
					{
						OutFailedChannelIdentifiers.AddUnique(RequestedBufferSize.ChannelIdentifier);
						Result = EResizeBufferResult::BufferCouldNotBeResize;
					}
				}
			}
			if (Result == EResizeBufferResult::BufferCouldNotBeResize)
			{
				return Result;
			}
		}

		return EResizeBufferResult::Success;
	}
}

FTimedDataMonitorCalibration::~FTimedDataMonitorCalibration()
{
	Reset();
}

void FTimedDataMonitorCalibration::Reset()
{
	NumberOfRetry = 0;
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}

	if (OnCalibrationCompleted.IsBound())
	{
		FOnCalibrationCompletedSignature PreviousOnCalibrationCompleted = OnCalibrationCompleted;
		OnCalibrationCompleted.Unbind();

		// Client may destroy the object in the callback. No memory access after the callback
		FTimedDataMonitorCalibrationResult Result;
		Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_Reset;
		PreviousOnCalibrationCompleted.Execute(Result);
	}
}


bool FTimedDataMonitorCalibration::IsRunning() const
{
	return TickerHandle.IsValid();
}


void FTimedDataMonitorCalibration::CalibrateWithTimecode(const FTimedDataMonitorCalibrationParameters& Parameters, FOnCalibrationCompletedSignature OnCompleted)
{
	Reset();

	CalibrationParameters = Parameters;
	OnCalibrationCompleted = OnCompleted;

	RunCalibrateWithTimecode_CheckForReset(0.f);
}


bool FTimedDataMonitorCalibration::RunCalibrateWithTimecode_CheckForReset(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FTimedDataMonitorCalibration_RunCalibrateWithTimecode_CheckForReset);

	if (CalibrationParameters.bUseStandardDeviation && CalibrationParameters.bResetStatisticsBeforeUsingStandardDeviation)
	{
		GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>()->ResetAllBufferStats();
			
		FTickerDelegate Delegate = FTickerDelegate::CreateRaw(this, &FTimedDataMonitorCalibration::RunCalibrateWithTimecode);
		float Timer = CalibrationParameters.AmountOfSecondsToWaitAfterStatisticReset;
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(Delegate, Timer);
	}
	else
	{
		RunCalibrateWithTimecode(0.f);
	}
	return false;
}


bool FTimedDataMonitorCalibration::RunCalibrateWithTimecode(float)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FTimedDataMonitorCalibration_RunCalibrateWithTimecode);

	FTimedDataMonitorCalibrationResult Result = CalibrateWithTimecode(CalibrationParameters);
	bool bCallcallback = true;
	if (Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Retry_IncreaseBufferSize
		|| Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Retry_NotEnoughData)
	{
		++NumberOfRetry;
		if (NumberOfRetry < CalibrationParameters.NumberOfRetries)
		{
			bCallcallback = false;
			FTickerDelegate Delegate = FTickerDelegate::CreateRaw(this, &FTimedDataMonitorCalibration::RunCalibrateWithTimecode_CheckForReset);
			float Timer = 1.f; // todo find a better way to handle this. how much time should we wait...
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(Delegate, Timer);
		}

	}
	
	if (bCallcallback)
	{
		FOnCalibrationCompletedSignature PreviousOnCalibrationCompleted = OnCalibrationCompleted;
		OnCalibrationCompleted.Unbind();
		TickerHandle.Reset();

		// Client may destroy the object in the callback. No memory access after the callback
		PreviousOnCalibrationCompleted.ExecuteIfBound(MoveTemp(Result));
	}
	return false;
}


// With [A,B,C,D] We are not able to calibrate. (C:100-11 is too big of a gab)
// With [A,D] We need to increase the buffer size of A2 (48-11), D1 and D2. Set the TimecodeProvider offset to 39 (50-11)
FTimedDataMonitorCalibrationResult FTimedDataMonitorCalibration::CalibrateWithTimecode(const FTimedDataMonitorCalibrationParameters& CalibrationParameters)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	FTimedDataMonitorCalibrationResult Result;

	UTimecodeProvider* CurrentTimecodeProvider = GEngine->GetTimecodeProvider();
	FQualifiedFrameTime CurrentFrameTime;
	if (CurrentTimecodeProvider == nullptr
		|| CurrentTimecodeProvider->GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized
		|| !FApp::GetCurrentFrameTime().IsSet())
	{
		Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_NoTimecode;
		return Result;
	}
	CurrentFrameTime = CurrentTimecodeProvider->GetQualifiedFrameTime(); // Without offset

	const TArray<TimedDataCalibration::FEnabledInput> AllValidInputIndentifiers = TimedDataCalibration::GetEnabledInputs(TimedDataMonitorSubsystem);

	// Are they all connected?
	for (const TimedDataCalibration::FEnabledInput& EnabledInput : AllValidInputIndentifiers)
	{
		if (TimedDataMonitorSubsystem->GetInputConnectionState(EnabledInput.InputIdentifier) != ETimedDataInputState::Connected)
		{
			Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_UnresponsiveInput;
			Result.FailureInputIdentifiers.Add(EnabledInput.InputIdentifier);
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Failed_UnresponsiveInput)
	{
		return Result;
	}

	// Are they in evaluation type timecode?
	for (const TimedDataCalibration::FEnabledInput& EnabledInput : AllValidInputIndentifiers)
	{
		if (TimedDataMonitorSubsystem->GetInputEvaluationType(EnabledInput.InputIdentifier) != ETimedDataInputEvaluationType::Timecode)
		{
			Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_InvalidEvaluationType;
			Result.FailureInputIdentifiers.Add(EnabledInput.InputIdentifier);
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Failed_InvalidEvaluationType)
	{
		return Result;
	}

	// Do they have invalid frame rate?
	for (const TimedDataCalibration::FEnabledInput& EnabledInput : AllValidInputIndentifiers)
	{
		if (TimedDataMonitorSubsystem->GetInputFrameRate(EnabledInput.InputIdentifier) == ITimedDataInput::UnknownFrameRate)
		{
			Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_InvalidFrameRate;
			Result.FailureInputIdentifiers.Add(EnabledInput.InputIdentifier);
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Failed_InvalidFrameRate)
	{
		return Result;
	}

	// Do they all have buffers?
	for (const TimedDataCalibration::FEnabledInput& EnabledInput : AllValidInputIndentifiers)
	{
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : EnabledInput.ChannelIdentifiers)
		{
			if (TimedDataMonitorSubsystem->GetChannelNumberOfSamples(ChannelIdentifier) <= 0)
			{
				Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Failed_NoDataBuffered;
				Result.FailureInputIdentifiers.AddUnique(EnabledInput.InputIdentifier);
			}
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorCalibrationReturnCode::Failed_NoDataBuffered)
	{
		return Result;
	}

	const int32 NumberOfSTD = CalibrationParameters.bUseStandardDeviation ? CalibrationParameters.NumberOfStandardDeviation : 0;

	// Collect the min and max of all inputs
	TArray<TimedDataCalibration::FChannelSampleMinMax> AllInputsSampleMinMax;
	for (const TimedDataCalibration::FEnabledInput& EnabledInput : AllValidInputIndentifiers)
	{
		float EvaluationOffsetInSeconds = TimedDataMonitorSubsystem->GetInputEvaluationOffsetInSeconds(EnabledInput.InputIdentifier);
		AllInputsSampleMinMax.Append(TimedDataCalibration::GetChannelsMinMax(TimedDataMonitorSubsystem, ETimedDataInputEvaluationType::Timecode, EnabledInput.ChannelIdentifiers, EvaluationOffsetInSeconds, NumberOfSTD));
	}

	// Test if all the samples are in the range of the EvaluationTime
	const double EvaluationTime = CurrentFrameTime.AsSeconds();
	const bool bAllChannelInRangeOfEvaluationTime = TimedDataCalibration::IsInRange(AllInputsSampleMinMax, EvaluationTime);
	if (bAllChannelInRangeOfEvaluationTime)
	{
		CurrentTimecodeProvider->FrameDelay = 0.0f;
		Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Succeeded;
		return Result;
	}

	TimedDataCalibration::FSmallestBiggestSample InputsSmallestBiggestSample =
		TimedDataCalibration::GetSmallestBiggestSample(AllInputsSampleMinMax);
	const double CalibrationTargetSeconds = InputsSmallestBiggestSample.SmallestMaxInSeconds;

	// Is there a range of data that everyone is happy with [A,D] == 11
	TArray<TimedDataCalibration::FChannelSampleMinMax> NotInRanges;
	for (const TimedDataCalibration::FChannelSampleMinMax& SampleMinMax : AllInputsSampleMinMax)
	{
		if (!TimedDataCalibration::IsInRange(SampleMinMax, CalibrationTargetSeconds))
		{
			NotInRanges.Add(SampleMinMax);
		}
	}
	if (NotInRanges.Num() == 0)
	{
		// Set the new offset of the TC provider
		const double OffsetInSeconds = EvaluationTime - CalibrationTargetSeconds; // If [A,D], 50-11=39
		CurrentTimecodeProvider->FrameDelay = FMath::CeilToFloat(ITimedDataInput::ConvertSecondOffsetInFrameOffset(OffsetInSeconds, CurrentFrameTime.Rate)); // for media, get a full frame
		Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Succeeded;
		return Result;
	}


	// For those that are not in range, can we increase the range
	if (!CalibrationParameters.bBufferResizeAllowed)
	{
		Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Retry_IncreaseBufferSize;
		return Result;
	}

	// Resize all the buffer or only those that needs it
	TArray<TimedDataCalibration::FInputBufferSize> RequestedInputSizes;
	TArray<TimedDataCalibration::FChannelBufferSize> RequestedChannelSizes;
	{
		const TArray<TimedDataCalibration::FChannelSampleMinMax>& BuffersToResize = CalibrationParameters.bBufferShrinkAllowed ? AllInputsSampleMinMax : NotInRanges;

		// For each input/ channel, how much buffer size do we need
		TArray<TimedDataCalibration::FChannelBufferSize> NewChannelBufferSizes = TimedDataCalibration::GetNewBufferSize(TimedDataMonitorSubsystem, ETimedDataInputEvaluationType::Timecode, BuffersToResize, CalibrationTargetSeconds, NumberOfSTD);
		TimedDataCalibration::SplitBufferSizeForDoesControlledByInput(TimedDataMonitorSubsystem, NewChannelBufferSizes, RequestedInputSizes, RequestedChannelSizes);

		// Resize the buffers
		TArray<FTimedDataMonitorInputIdentifier> ResizeFailedInputIdendifiers;
		TArray<FTimedDataMonitorChannelIdentifier> ResizeFailedChannelIdentifiers;
		TimedDataCalibration::EResizeBufferResult ResizeResult = TimedDataCalibration::ResizeBuffers(TimedDataMonitorSubsystem, RequestedInputSizes, RequestedChannelSizes
			, CalibrationParameters.bBufferShrinkAllowed, CalibrationParameters.bFailedIfBufferCantBeResize
			, ResizeFailedInputIdendifiers, ResizeFailedChannelIdentifiers);

		// If it failed, translate the error code and return
		if (ResizeResult == TimedDataCalibration::EResizeBufferResult::BufferCouldNotBeResize || ResizeResult == TimedDataCalibration::EResizeBufferResult::NotEnoughData)
		{
			Result.FailureInputIdentifiers = MoveTemp(ResizeFailedInputIdendifiers);
			for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : ResizeFailedChannelIdentifiers)
			{
				Result.FailureInputIdentifiers.AddUnique(TimedDataMonitorSubsystem->GetChannelInput(ChannelIdentifier));
			}
			Result.ReturnCode = ResizeResult == TimedDataCalibration::EResizeBufferResult::BufferCouldNotBeResize
				? ETimedDataMonitorCalibrationReturnCode::Failed_BufferCouldNotBeResize
				: ETimedDataMonitorCalibrationReturnCode::Retry_NotEnoughData;
			return Result;
		}
	}

	// Retry after some times if it's better with the new buffer size
	for (const TimedDataCalibration::FInputBufferSize& RequestedBufferSize : RequestedInputSizes)
	{
		Result.FailureInputIdentifiers.Add(RequestedBufferSize.InputIdentifier);
	}
	for (const TimedDataCalibration::FChannelBufferSize& RequestedBufferSize : RequestedChannelSizes)
	{
		Result.FailureInputIdentifiers.AddUnique(TimedDataMonitorSubsystem->GetChannelInput(RequestedBufferSize.ChannelIdentifier));
	}
	Result.ReturnCode = ETimedDataMonitorCalibrationReturnCode::Retry_IncreaseBufferSize;
	return Result;
}




// For InputA, we should increase the buffer size of A2 (48-11) and set an offset so that 11 == 50
// For InputB, we set an offset so that 100 == 50
// For InputC, we cannot find anything since the difference is too big. (100 - 11) Failed.
// For InputD, we set an offset so that 50 == 50
FTimedDataMonitorTimeCorrectionResult FTimedDataMonitorCalibration::ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& InputIdentifier, const FTimedDataMonitorTimeCorrectionParameters& TimeCorrectionParameters)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	FTimedDataMonitorTimeCorrectionResult Result;

	if (!TimedDataMonitorSubsystem->DoesInputExist(InputIdentifier) || TimedDataMonitorSubsystem->GetInputEnabled(InputIdentifier) == ETimedDataMonitorInputEnabled::Disabled)
	{
		Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Failed_InvalidInput;
		return Result;
	}

	const ETimedDataInputEvaluationType EvaluationType = TimedDataMonitorSubsystem->GetInputEvaluationType(InputIdentifier);
	const double CurrentPlatformTime = FApp::GetCurrentTime();
	UTimecodeProvider* CurrentTimecodeProvider = GEngine->GetTimecodeProvider();
	FQualifiedFrameTime CurrentFrameTime;
	if (EvaluationType == ETimedDataInputEvaluationType::Timecode)
	{
		if (CurrentTimecodeProvider == nullptr || CurrentTimecodeProvider->GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized || !FApp::GetCurrentFrameTime().IsSet())
		{
			Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Failed_NoTimecode;
			return Result;
		}

		CurrentFrameTime = FApp::GetCurrentFrameTime().GetValue(); // With offset
	}

	TArray<FTimedDataMonitorChannelIdentifier> AllValidChannelIdentifiers = TimedDataCalibration::GetEnabledChannelIdentifiers(TimedDataMonitorSubsystem, InputIdentifier);

	// Test all Channels for Failed_UnresponsiveInput
	for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : AllValidChannelIdentifiers)
	{
		if (TimedDataMonitorSubsystem->GetChannelConnectionState(ChannelIdentifier) != ETimedDataInputState::Connected)
		{
			Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Failed_UnresponsiveInput;
			Result.FailureChannelIdentifiers.Add(ChannelIdentifier);
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorTimeCorrectionReturnCode::Failed_UnresponsiveInput)
	{
		return Result;
	}

	//Don't touch inputs with evaluation type set to none. Just consider it a success
	if (EvaluationType == ETimedDataInputEvaluationType::None)
	{
		Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Succeeded;
		return Result;
	}

	// Do they all have buffers?
	for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : AllValidChannelIdentifiers)
	{
		if (TimedDataMonitorSubsystem->GetChannelNumberOfSamples(ChannelIdentifier) <= 0)
		{
			Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Failed_NoDataBuffered;
			Result.FailureChannelIdentifiers.AddUnique(ChannelIdentifier);
		}
	}
	if (Result.ReturnCode == ETimedDataMonitorTimeCorrectionReturnCode::Failed_NoDataBuffered)
	{
		return Result;
	}

	const int32 NumberOfSTD = TimeCorrectionParameters.bUseStandardDeviation ? TimeCorrectionParameters.NumberOfStandardDeviation : 0;

	// Collect all DataTimes
	float EvaluationOffsetInSeconds = 0.f; // We want to calculate without Evaluation Time (the goal is to find the offset)
	TArray<TimedDataCalibration::FChannelSampleMinMax> ChannelSamplesMinMax = TimedDataCalibration::GetChannelsMinMax(TimedDataMonitorSubsystem, EvaluationType, AllValidChannelIdentifiers, EvaluationOffsetInSeconds, NumberOfSTD);
	TimedDataCalibration::FSmallestBiggestSample SmallestBiggestSample = TimedDataCalibration::GetSmallestBiggestSample(ChannelSamplesMinMax);

	// Find what section that matches for each channel
	const double EvaluationTime = EvaluationType == ETimedDataInputEvaluationType::Timecode ? CurrentFrameTime.AsSeconds() : CurrentPlatformTime;

	// Test if all the samples are in the range of the EvaluationTime 
	const bool bAllChannelInRangeOfEvaluationTime = TimedDataCalibration::IsInRange(ChannelSamplesMinMax, EvaluationTime);
	if (bAllChannelInRangeOfEvaluationTime)
	{
		// Set the evaluation offset for later (case D)
		TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(InputIdentifier, 0.0);

		Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Succeeded;
		return Result;
	}

	const double TimeCorrectionTargetSeconds = SmallestBiggestSample.SmallestMaxInSeconds;

	// Is there a range of data that everyone is happy with
	TArray<TimedDataCalibration::FChannelSampleMinMax> NotInRanges;
	for (const TimedDataCalibration::FChannelSampleMinMax& SampleMinMax : ChannelSamplesMinMax)
	{
		if (!TimedDataCalibration::IsInRange(SampleMinMax, TimeCorrectionTargetSeconds))
		{
			NotInRanges.Add(SampleMinMax);
		}
	}
	if (NotInRanges.Num() == 0)
	{
		// Set the evaluation offset for later (case B)
		const double NewOffset = EvaluationTime - TimeCorrectionTargetSeconds;
		TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(InputIdentifier, NewOffset);

		Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Succeeded;
		return Result;
	}

	// For those that are not in range, can we increase the range
	if (!TimeCorrectionParameters.bBufferResizeAllowed)
	{
		Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Retry_IncreaseBufferSize;
		return Result;
	}

	// For each input/ channel, how much buffer size do we need
	{
		// Resize all the buffer or only those that needs it
		const TArray<TimedDataCalibration::FChannelSampleMinMax>& BuffersToResize = TimeCorrectionParameters.bBufferShrinkAllowed ? ChannelSamplesMinMax : NotInRanges;

		TArray<TimedDataCalibration::FChannelBufferSize> NewChannelBufferSizes =
			TimedDataCalibration::GetNewBufferSize(TimedDataMonitorSubsystem, EvaluationType, BuffersToResize, TimeCorrectionTargetSeconds, NumberOfSTD);

		// Resize the buffers

		TArray<TimedDataCalibration::FInputBufferSize> RequestedInputSizes;
		TArray<FTimedDataMonitorInputIdentifier> ResizeFailedInputIdendifiers;
		TArray<FTimedDataMonitorChannelIdentifier> ResizeFailedChannelIdentifiers;
		TimedDataCalibration::EResizeBufferResult ResizeResult = TimedDataCalibration::ResizeBuffers(TimedDataMonitorSubsystem, RequestedInputSizes, NewChannelBufferSizes
			, TimeCorrectionParameters.bBufferShrinkAllowed, TimeCorrectionParameters.bFailedIfBufferCantBeResize
			, ResizeFailedInputIdendifiers, ResizeFailedChannelIdentifiers);

		// If it failed, translate the error code and return
		if (ResizeResult == TimedDataCalibration::EResizeBufferResult::BufferCouldNotBeResize || ResizeResult == TimedDataCalibration::EResizeBufferResult::NotEnoughData)
		{
			Result.FailureChannelIdentifiers = MoveTemp(ResizeFailedChannelIdentifiers);
			Result.ReturnCode = ResizeResult == TimedDataCalibration::EResizeBufferResult::BufferCouldNotBeResize
				? ETimedDataMonitorTimeCorrectionReturnCode::Failed_BufferCouldNotBeResize
				: ETimedDataMonitorTimeCorrectionReturnCode::Retry_NotEnoughData;
			return Result;
		}
	}

	// We found something but the buffer size need to be increased
	const double NewOffset = EvaluationTime - SmallestBiggestSample.SmallestMaxInSeconds ;
	TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(InputIdentifier, NewOffset);

	Result.ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Retry_IncreaseBufferSize;
	return Result;
}
