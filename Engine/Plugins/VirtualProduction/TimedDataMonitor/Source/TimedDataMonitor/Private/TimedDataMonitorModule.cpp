// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimedDataMonitorPrivate.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogTimedDataMonitor)

IMPLEMENT_MODULE(FDefaultModuleImpl, TimedDataMonitor);
