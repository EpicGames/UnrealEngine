// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimedDataMonitorSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "IStageDataProvider.h"
#include "ITimeManagementModule.h"
#include "LatentActions.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "TimedDataInputCollection.h"
#include "TimedDataMonitorCalibration.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Misc/FileHelper.h"
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)


static TAutoConsoleVariable<bool> CVarEnableTimedDataMonitorSubsystemStats(TEXT("TimedDataMonitor.EnableStatUpdate"), 1, TEXT("Enable calculating evaluation statistics of all registered channels."));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
bool bIsTimedDataStatFileLoggingStarted = false;
static FAutoConsoleCommand TimedDataStartFileLoggingCmd(
	TEXT("TimedDataMonitor.StartFileLogging"),
	TEXT("Starts logging data for each channel for each frame to be logged in a file."),
	FConsoleCommandDelegate::CreateLambda([]() { bIsTimedDataStatFileLoggingStarted = true; })
);

static FAutoConsoleCommand TimedDataDisableFileLoggingCmd(
	TEXT("TimedDataMonitor.StopFileLogging"),
	TEXT("Stops logging data and dump everything to a file."),
	FConsoleCommandDelegate::CreateLambda([]() { bIsTimedDataStatFileLoggingStarted = false; })
);
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

TAutoConsoleVariable<float> CVarTimedDataStatisticsWeight(
	TEXT("TimedDataMonitor.Statistics.Weight"),
	0.1f,
	TEXT("The weight used when tracking Mean and Variance of samples distance to evaluation time. Number closer to 1 will give more weight to latest values and won't filter out noise."),
	ECVF_Default
);


#define LOCTEXT_NAMESPACE "TimedDataMonitorSubsystem"


FString FTimedDataMonitorChannelConnectionStateEvent::ToString() const
{
	return FString::Printf(TEXT("Channel '%s.%s' is now '%s'."), *InputName, *ChannelName, *StaticEnum<ETimedDataInputState>()->GetNameStringByValue((int64)NewState));
}

FString FTimedDataMonitorChannelEvaluationStateEvent::ToString() const
{
	return FString::Printf(TEXT("Channel '%s.%s' is now '%s'."), *InputName, *ChannelName, *StaticEnum<ETimedDataMonitorEvaluationState>()->GetNameStringByValue((int64)NewState));
}

/**
 *
 */
FTimedDataMonitorInputIdentifier FTimedDataMonitorInputIdentifier::NewIdentifier()
{
	FTimedDataMonitorInputIdentifier Item;
	Item.Identifier = FGuid::NewGuid();
	return Item;
}


/**
 *
 */
FTimedDataMonitorChannelIdentifier FTimedDataMonitorChannelIdentifier::NewIdentifier()
{
	FTimedDataMonitorChannelIdentifier Item;
	Item.Identifier = FGuid::NewGuid();
	return Item;
}


/**
 * 
 */
void UTimedDataMonitorSubsystem::FTimeDataInputItem::ResetValue()
{
	ChannelIdentifiers.Reset();
}

/**
 *
 */
void UTimedDataMonitorSubsystem::FTimeDataChannelItem::ResetValue()
{
	Statistics.Reset();
}

/**
 * 
 */
void UTimedDataMonitorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	const bool bInitializeTimedDataMonitor = GConfig->GetBoolOrDefault(TEXT("TimedDataMonitor"), TEXT("bEnableTimedDataMonitor"), true, GEngineIni);
	if (bInitializeTimedDataMonitor)
	{
		bRequestSourceListRebuilt = true;
		ITimeManagementModule::Get().GetTimedDataInputCollection().OnCollectionChanged().AddUObject(this, &UTimedDataMonitorSubsystem::OnTimedDataSourceCollectionChanged);
		FCoreDelegates::OnEndFrame.AddUObject(this, &UTimedDataMonitorSubsystem::EndFrameCallback);
		FCoreDelegates::OnBeginFrame.AddUObject(this, &UTimedDataMonitorSubsystem::BeginFrameCallback);
	}
}


void UTimedDataMonitorSubsystem::Deinitialize()
{
	const bool bInitializeTimedDataMonitor = GConfig->GetBoolOrDefault(TEXT("TimedDataMonitor"), TEXT("bEnableTimedDataMonitor"), true, GEngineIni);

	if (bInitializeTimedDataMonitor)
	{
		FCoreDelegates::OnBeginFrame.RemoveAll(this);
		FCoreDelegates::OnEndFrame.RemoveAll(this);

		if (ITimeManagementModule::IsAvailable())
		{
			ITimeManagementModule::Get().GetTimedDataInputCollection().OnCollectionChanged().RemoveAll(this);
		}

		bRequestSourceListRebuilt = true;
		InputMap.Reset();
		ChannelMap.Reset();
		OnIdentifierListChanged_Delegate.Broadcast();
		OnIdentifierListChanged_Dynamic.Broadcast();
	}

	Super::Deinitialize();
}


void UTimedDataMonitorSubsystem::BeginFrameCallback()
{
	UTimecodeProvider* CurrentTimecodeProvider = GEngine->GetTimecodeProvider();
	const TOptional<FQualifiedFrameTime> CurrentFrameTimeOffsetted = FApp::GetCurrentFrameTime();
	if (CurrentTimecodeProvider == nullptr
		|| CurrentTimecodeProvider->GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized
		|| !CurrentFrameTimeOffsetted.IsSet())
	{
		CachedTimecodeProviderFrameDelayInSeconds = 0.0f;
	}
	else
	{
		CachedTimecodeProviderFrameDelayInSeconds = CurrentTimecodeProvider->GetFrameRate().AsInterval() * CurrentTimecodeProvider->FrameDelay;
	}
}

void UTimedDataMonitorSubsystem::EndFrameCallback()
{
	const bool bUpdateStats = CVarEnableTimedDataMonitorSubsystemStats.GetValueOnGameThread();
	if (bUpdateStats)
	{
		UpdateStatFileLoggingState();
		UpdateEvaluationStatistics();
	}
	
	BuildSourcesListIfNeeded();
	CacheConnectionState();
	CacheEvaluationState();
}

ITimedDataInput* UTimedDataMonitorSubsystem::GetTimedDataInput(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input;
	}

	return nullptr;
}


ITimedDataInputChannel* UTimedDataMonitorSubsystem::GetTimedDataChannel(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataChannelItem* ChannelItem = ChannelMap.Find(Identifier))
	{
		return ChannelItem->Channel;
	}

	return nullptr;
}


float UTimedDataMonitorSubsystem::GetEvaluationTimeOffsetInSeconds(ETimedDataInputEvaluationType EvaluationType)
{
	float Result = 0.0f;

	switch (EvaluationType)
	{
	case ETimedDataInputEvaluationType::Timecode:
	{
		Result = CachedTimecodeProviderFrameDelayInSeconds;
		break;
	}
	case ETimedDataInputEvaluationType::PlatformTime:
	case ETimedDataInputEvaluationType::None:
	default:
		break;
	}

	return Result;
}

double UTimedDataMonitorSubsystem::GetEvaluationTime(ETimedDataInputEvaluationType EvaluationType)
{
	double Result = 0.0;
	switch (EvaluationType)
	{
	case ETimedDataInputEvaluationType::Timecode:
		if (FApp::GetCurrentFrameTime().IsSet())
		{
			Result = FApp::GetCurrentFrameTime().GetValue().AsSeconds();
		}
		break;
	case ETimedDataInputEvaluationType::PlatformTime:
		Result = FApp::GetCurrentTime();
		break;
	case ETimedDataInputEvaluationType::None:
	default:
	break;
	}
	return Result;
}


TArray<FTimedDataMonitorInputIdentifier> UTimedDataMonitorSubsystem::GetAllInputs()
{
	BuildSourcesListIfNeeded();

	TArray<FTimedDataMonitorInputIdentifier> Result;
	InputMap.GenerateKeyArray(Result);
	return Result;
}


TArray<FTimedDataMonitorChannelIdentifier> UTimedDataMonitorSubsystem::GetAllChannels()
{
	BuildSourcesListIfNeeded();

	TArray<FTimedDataMonitorChannelIdentifier> Result;
	ChannelMap.GenerateKeyArray(Result);
	return Result;
}


TArray<FTimedDataMonitorChannelIdentifier> UTimedDataMonitorSubsystem::GetAllEnabledChannels()
{
	BuildSourcesListIfNeeded();

	TArray<FTimedDataMonitorChannelIdentifier> Result;
	Result.Reset(ChannelMap.Num());
	for (const auto& ChannelMapItt : ChannelMap)
	{
		if (ChannelMapItt.Value.bEnabled)
		{
			Result.Add(ChannelMapItt.Key);
		}
	}
	return Result;
}


void UTimedDataMonitorSubsystem::CalibrateLatent(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, const FTimedDataMonitorCalibrationParameters& CalibrationParameters, FTimedDataMonitorCalibrationResult& Result)
{
	struct FCalibrateAction : public FPendingLatentAction
	{
	public:
		FName ExecutionFunction;
		int32 Linkage;
		FWeakObjectPtr CallbackTarget;
		FTimedDataMonitorCalibrationResult& Result;
		TUniquePtr<FTimedDataMonitorCalibration> Calibration;
		bool bOnCompleted;

		FCalibrateAction(const FLatentActionInfo& InLatentInfo, FTimedDataMonitorCalibrationResult& InResult)
			: FPendingLatentAction()
			, ExecutionFunction(InLatentInfo.ExecutionFunction)
			, Linkage(InLatentInfo.Linkage)
			, CallbackTarget(InLatentInfo.CallbackTarget)
			, Result(InResult)
			, Calibration(new FTimedDataMonitorCalibration)
			, bOnCompleted(false)
		{
		}

		virtual void UpdateOperation(FLatentResponse& Response) override
		{
			if (bOnCompleted)
			{
				Response.FinishAndTriggerIf(true, ExecutionFunction, Linkage, CallbackTarget);
			}
		}

#if WITH_EDITOR
		virtual FString GetDescription() const override
		{
			return FString::Printf(TEXT("Calibrating."));
		}
#endif

		void OnCompleted(FTimedDataMonitorCalibrationResult InResult)
		{
			Result = InResult;
			bOnCompleted = true;
		}
	};

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FCalibrateAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FCalibrateAction* NewAction = new FCalibrateAction(LatentInfo, Result);
			NewAction->Calibration->CalibrateWithTimecode(CalibrationParameters, FTimedDataMonitorCalibration::FOnCalibrationCompletedSignature::CreateRaw(NewAction, &FCalibrateAction::OnCompleted));
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("The calibration is already running."), ELogVerbosity::Warning, "CalibrationActionAlreadyStarted");
		}
	}
};


FTimedDataMonitorTimeCorrectionResult UTimedDataMonitorSubsystem::ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& InputIdentifier, const FTimedDataMonitorTimeCorrectionParameters& TimeCorrectionParameters)
{
	return FTimedDataMonitorCalibration::ApplyTimeCorrection(InputIdentifier, TimeCorrectionParameters);
}


void UTimedDataMonitorSubsystem::ResetAllBufferStats()
{
	BuildSourcesListIfNeeded();

	for (auto& ChannelItt : ChannelMap)
	{
		ChannelItt.Value.Channel->ResetBufferStats();
		ChannelItt.Value.ResetValue();
	}
}


ETimedDataMonitorEvaluationState UTimedDataMonitorSubsystem::GetEvaluationState()
{
	return CachedEvaluationState;
}


bool UTimedDataMonitorSubsystem::DoesInputExist(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	return InputMap.Contains(Identifier);
}


ETimedDataMonitorInputEnabled UTimedDataMonitorSubsystem::GetInputEnabled(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		int32 bCountEnabled = 0;
		int32 bCountDisabled = 0;
		for (const FTimedDataMonitorChannelIdentifier& Channel : InputItem->ChannelIdentifiers)
		{
			if (ChannelMap[Channel].bEnabled)
			{
				++bCountEnabled;
				if (bCountDisabled > 0)
				{
					return ETimedDataMonitorInputEnabled::MultipleValues;
				}
			}
			else
			{
				++bCountDisabled;
				if (bCountEnabled > 0)
				{
					return ETimedDataMonitorInputEnabled::MultipleValues;
				}
			}
		}
		return bCountEnabled > 0 ? ETimedDataMonitorInputEnabled::Enabled : ETimedDataMonitorInputEnabled::Disabled;
	}

	return ETimedDataMonitorInputEnabled::Disabled;
}


void UTimedDataMonitorSubsystem::SetInputEnabled(const FTimedDataMonitorInputIdentifier& Identifier, bool bInEnabled)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		for (const FTimedDataMonitorChannelIdentifier& ChannelId : InputItem->ChannelIdentifiers)
		{
			ChannelMap[ChannelId].bEnabled = bInEnabled;
		}
	}
}


FText UTimedDataMonitorSubsystem::GetInputDisplayName(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->GetDisplayName();
	}

	return FText::GetEmpty();
}


TArray<FTimedDataMonitorChannelIdentifier> UTimedDataMonitorSubsystem::GetInputChannels(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->ChannelIdentifiers;
	}

	return TArray<FTimedDataMonitorChannelIdentifier>();
}


ETimedDataInputEvaluationType UTimedDataMonitorSubsystem::GetInputEvaluationType(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->GetEvaluationType();
	}

	return ETimedDataInputEvaluationType::None;
}


void UTimedDataMonitorSubsystem::SetInputEvaluationType(const FTimedDataMonitorInputIdentifier& Identifier, ETimedDataInputEvaluationType Evaluation)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->SetEvaluationType(Evaluation);
	}
}


float UTimedDataMonitorSubsystem::GetInputEvaluationOffsetInSeconds(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return (float)SourceItem->Input->GetEvaluationOffsetInSeconds();
	}

	return 0.f;
}


void UTimedDataMonitorSubsystem::SetInputEvaluationOffsetInSeconds(const FTimedDataMonitorInputIdentifier& Identifier, float Offset)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		if (!SourceItem->Input->SupportsSubFrames())
		{
			// If sub frames are not supported, convert to frames, round it, then re-convert to seconds in order to get
			// a time that aligns with a frame.
			const int32 RoundedFrameOffset = FMath::RoundToInt32(SourceItem->Input->ConvertSecondOffsetInFrameOffset(Offset));
			const double AlignedSecondsOffset = SourceItem->Input->ConvertFrameOffsetInSecondOffset(RoundedFrameOffset);
			return SourceItem->Input->SetEvaluationOffsetInSeconds(AlignedSecondsOffset);
		}
		return SourceItem->Input->SetEvaluationOffsetInSeconds(Offset);
	}
}

float UTimedDataMonitorSubsystem::GetInputEvaluationOffsetInFrames(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->ConvertSecondOffsetInFrameOffset(SourceItem->Input->GetEvaluationOffsetInSeconds());
	}

	return 0.f;
}


void UTimedDataMonitorSubsystem::SetInputEvaluationOffsetInFrames(const FTimedDataMonitorInputIdentifier& Identifier, float Offset)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->SetEvaluationOffsetInSeconds(SourceItem->Input->ConvertFrameOffsetInSecondOffset(Offset));
	}
}


FFrameRate UTimedDataMonitorSubsystem::GetInputFrameRate(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->GetFrameRate();
	}

	return ITimedDataInput::UnknownFrameRate;
}


FTimedDataChannelSampleTime UTimedDataMonitorSubsystem::GetInputOldestDataTime(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	FTimedDataChannelSampleTime ResultSampleTime(0.0, FQualifiedFrameTime());
	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		bool bFirstElement = true;
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : SourceItem->ChannelIdentifiers)
		{
			FTimedDataChannelSampleTime OldestSampleTime = ChannelMap[ChannelIdentifier].Channel->GetOldestDataTime();
			if (bFirstElement)
			{
				ResultSampleTime = OldestSampleTime;
				bFirstElement = false;
			}
			else
			{
				ResultSampleTime.PlatformSecond = FMath::Min(OldestSampleTime.PlatformSecond, ResultSampleTime.PlatformSecond);
				if (OldestSampleTime.Timecode.AsSeconds() < ResultSampleTime.Timecode.AsSeconds())
				{
					ResultSampleTime.Timecode = OldestSampleTime.Timecode;
				}
			}
		}
	}

	return ResultSampleTime;
}


FTimedDataChannelSampleTime UTimedDataMonitorSubsystem::GetInputNewestDataTime(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	FTimedDataChannelSampleTime ResultSampleTime(0.0, FQualifiedFrameTime());
	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		for (FTimedDataMonitorChannelIdentifier ChannelIdentifier : SourceItem->ChannelIdentifiers)
		{
			FTimedDataChannelSampleTime NewestSampleTime = ChannelMap[ChannelIdentifier].Channel->GetNewestDataTime();
			ResultSampleTime.PlatformSecond = FMath::Max(NewestSampleTime.PlatformSecond, ResultSampleTime.PlatformSecond);
			if (NewestSampleTime.Timecode.AsSeconds() > ResultSampleTime.Timecode.AsSeconds())
			{
				ResultSampleTime.Timecode = NewestSampleTime.Timecode;
			}
		}
	}

	return ResultSampleTime;
}


bool UTimedDataMonitorSubsystem::IsDataBufferSizeControlledByInput(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->IsDataBufferSizeControlledByInput();
	}

	return false;
}


int32 UTimedDataMonitorSubsystem::GetInputDataBufferSize(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		return SourceItem->Input->GetDataBufferSize();
	}

	return 0;
}


void UTimedDataMonitorSubsystem::SetInputDataBufferSize(const FTimedDataMonitorInputIdentifier& Identifier, int32 BufferSize)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataInputItem* SourceItem = InputMap.Find(Identifier))
	{
		if (SourceItem->Input->IsDataBufferSizeControlledByInput())
		{
			SourceItem->Input->SetDataBufferSize(BufferSize);
		}
	}
}

ETimedDataInputState UTimedDataMonitorSubsystem::GetInputConnectionState(const FTimedDataMonitorInputIdentifier& Identifier)
{
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		return InputItem->CachedConnectionState;
	}

	return ETimedDataInputState::Disconnected;
}


ETimedDataMonitorEvaluationState UTimedDataMonitorSubsystem::GetInputEvaluationState(const FTimedDataMonitorInputIdentifier& Identifier)
{
	ETimedDataMonitorEvaluationState WorstState = ETimedDataMonitorEvaluationState::Disabled;
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		return InputItem->CachedEvaluationState;
	}

	return WorstState;
}


float UTimedDataMonitorSubsystem::GetInputEvaluationDistanceToNewestSampleMean(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();
	
	float WorstNewestMean = 0.f;
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		WorstNewestMean = TNumericLimits<float>::Lowest();
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem->ChannelIdentifiers)
		{
			WorstNewestMean = FMath::Max(ChannelMap[ChannelIdentifier].Statistics.NewestSampleDistanceTracker.CurrentMean, WorstNewestMean);
		}
	}

	return WorstNewestMean;
}


float UTimedDataMonitorSubsystem::GetInputEvaluationDistanceToOldestSampleMean(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	float WorstOldesttMean = 0.f;
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		WorstOldesttMean = TNumericLimits<float>::Max();
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem->ChannelIdentifiers)
		{
			WorstOldesttMean = FMath::Min(ChannelMap[ChannelIdentifier].Statistics.OldestSampleDistanceTracker.CurrentMean, WorstOldesttMean);
		}
	}

	return WorstOldesttMean;
}


float UTimedDataMonitorSubsystem::GetInputEvaluationDistanceToNewestSampleStandardDeviation(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	float WorstNewestSSD = 0.f;
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem->ChannelIdentifiers)
		{
			WorstNewestSSD = FMath::Max(ChannelMap[ChannelIdentifier].Statistics.NewestSampleDistanceTracker.CurrentSTD, WorstNewestSSD);
		}
	}

	return WorstNewestSSD;
}


float UTimedDataMonitorSubsystem::GetInputEvaluationDistanceToOldestSampleStandardDeviation(const FTimedDataMonitorInputIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	float WorstOldestSSD = 0.f;
	if (const FTimeDataInputItem* InputItem = InputMap.Find(Identifier))
	{
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem->ChannelIdentifiers)
		{
			WorstOldestSSD = FMath::Max(ChannelMap[ChannelIdentifier].Statistics.OldestSampleDistanceTracker.CurrentSTD, WorstOldestSSD);
		}
	}

	return WorstOldestSSD;
}


bool UTimedDataMonitorSubsystem::DoesChannelExist(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	return ChannelMap.Contains(Identifier);
}


bool UTimedDataMonitorSubsystem::IsChannelEnabled(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->bEnabled;
	}

	return false;
}


void UTimedDataMonitorSubsystem::SetChannelEnabled(const FTimedDataMonitorChannelIdentifier& Identifier, bool bInEnabled)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		SourceItem->bEnabled = bInEnabled;
	}
}


FTimedDataMonitorInputIdentifier UTimedDataMonitorSubsystem::GetChannelInput(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->InputIdentifier;
	}

	return FTimedDataMonitorInputIdentifier();
}


FText UTimedDataMonitorSubsystem::GetChannelDisplayName(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetDisplayName();
	}

	return FText::GetEmpty();
}


ETimedDataInputState UTimedDataMonitorSubsystem::GetChannelConnectionState(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	if (const FTimeDataChannelItem* ChannelItem = ChannelMap.Find(Identifier))
	{
		return ChannelItem->CachedConnectionState;
	}

	return ETimedDataInputState::Disconnected;
}


ETimedDataMonitorEvaluationState UTimedDataMonitorSubsystem::GetChannelEvaluationState(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->CachedEvaluationState;
	}

	return ETimedDataMonitorEvaluationState::Disabled;
}


FTimedDataChannelSampleTime UTimedDataMonitorSubsystem::GetChannelOldestDataTime(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetOldestDataTime();
	}

	return FTimedDataChannelSampleTime();
}


FTimedDataChannelSampleTime UTimedDataMonitorSubsystem::GetChannelNewestDataTime(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetNewestDataTime();
	}

	return FTimedDataChannelSampleTime();
}


TArray<FTimedDataChannelSampleTime> UTimedDataMonitorSubsystem::GetChannelFrameDataTimes(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	TArray<FTimedDataChannelSampleTime> SampleTimes;

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		SampleTimes = SourceItem->Channel->GetDataTimes();
	}

	return SampleTimes;
}


int32 UTimedDataMonitorSubsystem::GetChannelNumberOfSamples(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetNumberOfSamples();
	}

	return 0;
}


int32 UTimedDataMonitorSubsystem::GetChannelDataBufferSize(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		if (InputMap[SourceItem->InputIdentifier].Input->IsDataBufferSizeControlledByInput())
		{
			return InputMap[SourceItem->InputIdentifier].Input->GetDataBufferSize();
		}
		else
		{
			return SourceItem->Channel->GetDataBufferSize();
		}
	}

	return 0;
}


void UTimedDataMonitorSubsystem::SetChannelDataBufferSize(const FTimedDataMonitorChannelIdentifier& Identifier, int32 BufferSize)
{
	BuildSourcesListIfNeeded();

	if (FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		if (!InputMap[SourceItem->InputIdentifier].Input->IsDataBufferSizeControlledByInput())
		{
			SourceItem->Channel->SetDataBufferSize(BufferSize);
		}
	}
}


int32 UTimedDataMonitorSubsystem::GetChannelBufferUnderflowStat(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetBufferUnderflowStat();
	}

	return 0;
}

int32 UTimedDataMonitorSubsystem::GetChannelBufferOverflowStat(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetBufferOverflowStat();
	}

	return 0;
}

int32 UTimedDataMonitorSubsystem::GetChannelFrameDroppedStat(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Channel->GetFrameDroppedStat();
	}

	return 0;
}

float UTimedDataMonitorSubsystem::GetChannelEvaluationDistanceToNewestSampleMean(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Statistics.NewestSampleDistanceTracker.CurrentMean;
	}

	return 0.0f;
}

float UTimedDataMonitorSubsystem::GetChannelEvaluationDistanceToOldestSampleMean(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Statistics.OldestSampleDistanceTracker.CurrentMean;
	}

	return 0.0f;
}

float UTimedDataMonitorSubsystem::GetChannelEvaluationDistanceToNewestSampleStandardDeviation(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Statistics.NewestSampleDistanceTracker.CurrentSTD;
	}

	return 0.0f;
}

float UTimedDataMonitorSubsystem::GetChannelEvaluationDistanceToOldestSampleStandardDeviation(const FTimedDataMonitorChannelIdentifier& Identifier)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		return SourceItem->Statistics.OldestSampleDistanceTracker.CurrentSTD;
	}

	return 0.0f;
}

void UTimedDataMonitorSubsystem::GetChannelLastEvaluationDataStat(const FTimedDataMonitorChannelIdentifier& Identifier, FTimedDataInputEvaluationData& Result)
{
	BuildSourcesListIfNeeded();

	if (const FTimeDataChannelItem* SourceItem = ChannelMap.Find(Identifier))
	{
		SourceItem->Channel->GetLastEvaluationData(Result);
	}
}

void UTimedDataMonitorSubsystem::BuildSourcesListIfNeeded()
{
	if (bRequestSourceListRebuilt)
	{
		if (!ITimeManagementModule::IsAvailable())
		{
			InputMap.Reset();
			ChannelMap.Reset();
		}
		else
		{
			bRequestSourceListRebuilt = false;

			const TArray<ITimedDataInput*>& TimedDataInputs = ITimeManagementModule::Get().GetTimedDataInputCollection().GetInputs();

			// Regenerate the list of inputs
			{
				TArray<FTimedDataMonitorInputIdentifier> PreviousInputList;
				InputMap.GenerateKeyArray(PreviousInputList);

				for (ITimedDataInput* TimedData : TimedDataInputs)
				{
					if(TimedData == nullptr)
					{
						continue;
					}

					FTimedDataMonitorInputIdentifier FoundIdentifier;
					for (const auto& Itt : InputMap)
					{
						if (Itt.Value.Input == TimedData)
						{
							FoundIdentifier = Itt.Key;
						}
					}

					if (FoundIdentifier.IsValid())
					{
						PreviousInputList.RemoveSingleSwap(FoundIdentifier);
						InputMap[FoundIdentifier].ResetValue();
					}
					else
					{
						// if not found, add it to the list
						FTimeDataInputItem NewInput; 
						NewInput.Input = TimedData;
						InputMap.Add(FTimedDataMonitorInputIdentifier::NewIdentifier(), MoveTemp(NewInput));
					}
				}

				// Remove old inputs
				for (const FTimedDataMonitorInputIdentifier& Old : PreviousInputList)
				{
					InputMap.Remove(Old);
				}
			}

			// Regenerate the list of channels
			{
				TArray<FTimedDataMonitorChannelIdentifier> PreviousChannelList;
				ChannelMap.GenerateKeyArray(PreviousChannelList);

				for (auto& InputItt : InputMap)
				{
					TArray<FTimedDataMonitorChannelIdentifier> OldChannelIdentifiers = InputItt.Value.ChannelIdentifiers;
					TArray<ITimedDataInputChannel*> NewChannels = InputItt.Value.Input->GetChannels();
					for (ITimedDataInputChannel* Channel : NewChannels)
					{
						if (Channel == nullptr)
						{
							continue;
						}

						FTimedDataMonitorChannelIdentifier FoundIdentifier;
						for (const auto& ChannelItt : ChannelMap)
						{
							if (ChannelItt.Value.Channel == Channel)
							{
								FoundIdentifier = ChannelItt.Key;
							}
						}

						if (FoundIdentifier.IsValid())
						{
							PreviousChannelList.RemoveSingleSwap(FoundIdentifier);
							OldChannelIdentifiers.RemoveSingleSwap(FoundIdentifier);
							InputItt.Value.ChannelIdentifiers.AddUnique(FoundIdentifier);
							ChannelMap[FoundIdentifier].ResetValue();
						}
						else
						{
							FoundIdentifier = FTimedDataMonitorChannelIdentifier::NewIdentifier();

							FTimeDataChannelItem NewChannel;
							NewChannel.Channel = Channel;
							ChannelMap.Add(FoundIdentifier, MoveTemp(NewChannel));
						}

						ChannelMap[FoundIdentifier].InputIdentifier = InputItt.Key;
						InputItt.Value.ChannelIdentifiers.AddUnique(FoundIdentifier);
					}

					for (const FTimedDataMonitorChannelIdentifier& Old : OldChannelIdentifiers)
					{
						InputItt.Value.ChannelIdentifiers.RemoveSingleSwap(Old);
					}
				}

				// Remove old channels
				for (const FTimedDataMonitorChannelIdentifier& Old : PreviousChannelList)
				{
					ChannelMap.Remove(Old);
				}
			}
		}
	}
}


void UTimedDataMonitorSubsystem::OnTimedDataSourceCollectionChanged()
{
	bRequestSourceListRebuilt = true;

	bool bCallDelegate = false;
	// update map right away to not have dandling pointer
	{
		const TArray<ITimedDataInput*>& TimedDataInputs = ITimeManagementModule::Get().GetTimedDataInputCollection().GetInputs();
		TArray<FTimedDataMonitorInputIdentifier, TInlineAllocator<4>> InputToRemove;
		for (const auto& Itt : InputMap)
		{
			if (!TimedDataInputs.Contains(Itt.Value.Input))
			{
				InputToRemove.Add(Itt.Key);
			}
		}
		for (const FTimedDataMonitorInputIdentifier& Id : InputToRemove)
		{
			bCallDelegate = true;
			InputMap.Remove(Id);
		}
	}

	{
		const TArray<ITimedDataInputChannel*>& TimedDataChannels = ITimeManagementModule::Get().GetTimedDataInputCollection().GetChannels();
		TArray<FTimedDataMonitorChannelIdentifier, TInlineAllocator<4>> ChannelToRemove;
		for (const auto& Itt : ChannelMap)
		{
			if (!TimedDataChannels.Contains(Itt.Value.Channel))
			{
				ChannelToRemove.Add(Itt.Key);
			}
		}
		for (const FTimedDataMonitorChannelIdentifier& Id : ChannelToRemove)
		{
			bCallDelegate = true;
			ChannelMap.Remove(Id);
		}
	}

	OnIdentifierListChanged_Delegate.Broadcast();
	OnIdentifierListChanged_Dynamic.Broadcast();
}

void UTimedDataMonitorSubsystem::CacheEvaluationState()
{
	//Start with best state
	ETimedDataMonitorEvaluationState SystemWorstState = ETimedDataMonitorEvaluationState::Disabled;

	//Go over each inputs and each channel and cache their evaluation state
	//Subsystem connection state will be the worst of them all
	for (auto& InputItt : InputMap)
	{
		bool bHasAtLeastOneItem = false;
		FTimeDataInputItem& InputItem = InputItt.Value;

		//Default to a disabled state. Might have no channels or an eval type that don't need to be aligned
		ETimedDataMonitorEvaluationState InputWorstState = ETimedDataMonitorEvaluationState::Disabled;

		if (InputItem.ChannelIdentifiers.Num() > 0 && InputItem.Input->GetEvaluationType() != ETimedDataInputEvaluationType::None)
		{
			for (FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem.ChannelIdentifiers)
			{
				FTimeDataChannelItem& ChannelItem = ChannelMap[ChannelIdentifier];
				CacheChannelEvaluationState(ChannelItem);

				//if channel is enabled, use it to update the input's state
				if ((uint8)ChannelItem.CachedEvaluationState < (uint8)InputWorstState)
				{
					InputWorstState = ChannelItem.CachedEvaluationState;
				}
			}
		}

		InputItem.CachedEvaluationState = InputWorstState;

		//Update system worst state based on this input
		if ((uint8)InputWorstState < (uint8)SystemWorstState)
		{
			SystemWorstState = InputWorstState;
		}
	}

	//Update cached system value
	if (SystemWorstState != CachedEvaluationState)
	{
		CachedEvaluationState = SystemWorstState;
	}
}

void UTimedDataMonitorSubsystem::CacheConnectionState()
{
	//Start with best state
	ETimedDataInputState SystemWorstState = ETimedDataInputState::Connected;

	//Go over each inputs and each channel and cache their connection state
	//Subsystem connection state will be the worst of them all
	if (InputMap.Num() > 0)
	{
		for (auto& InputItt : InputMap)
		{
			bool bHasAtLeastOneItem = false;
			FTimeDataInputItem& InputItem = InputItt.Value;
			ETimedDataInputState InputWorstState = ETimedDataInputState::Connected;

			for (FTimedDataMonitorChannelIdentifier& ChannelIdentifier : InputItem.ChannelIdentifiers)
			{
				FTimeDataChannelItem& ChannelItem = ChannelMap[ChannelIdentifier];
				
				const ETimedDataInputState NewChannelState = ChannelItem.Channel->GetState();
				
				//if channel is enabled, use it to update the input's state
				if (ChannelItem.bEnabled)
				{
					bHasAtLeastOneItem = true;

					if ((uint8)NewChannelState < (uint8)InputWorstState)
					{
						InputWorstState = NewChannelState;
					}

					//When Channel is enabled for timed tracking and its state change, let the stage know about it
					if (NewChannelState != ChannelItem.CachedConnectionState)
					{
						IStageDataProvider::SendMessage<FTimedDataMonitorChannelConnectionStateEvent>(EStageMessageFlags::Reliable, NewChannelState, InputItem.Input->GetDisplayName().ToString(), ChannelItem.Channel->GetDisplayName().ToString());
					}
				}

				ChannelItem.CachedConnectionState = NewChannelState;
			}
		
			InputItem.CachedConnectionState = InputWorstState;
			
			//Update system worst state based on this input
			if (bHasAtLeastOneItem && (uint8)InputWorstState < (uint8)SystemWorstState)
			{
				SystemWorstState = InputWorstState;
			}
		}
	}
	
	CachedConnectionState = SystemWorstState;
}

void UTimedDataMonitorSubsystem::CacheChannelEvaluationState(FTimeDataChannelItem& ChannelItem)
{
	ETimedDataMonitorEvaluationState NewState;
	const ITimedDataInput* Input = InputMap[ChannelItem.InputIdentifier].Input;
	check(Input);

	if (ChannelItem.Channel->GetState() != ETimedDataInputState::Connected || !ChannelItem.bEnabled)
	{
		NewState = ETimedDataMonitorEvaluationState::Disabled;
	}
	else if (ChannelItem.Channel->GetNumberOfSamples() <= 0)
	{
		NewState = ETimedDataMonitorEvaluationState::NoSample;
	}
	else
	{
		const ETimedDataInputEvaluationType EvaluationType = Input->GetEvaluationType();
		const double EvaluationOffset = Input->GetEvaluationOffsetInSeconds();
		const double OldestSampleTime = ChannelItem.Channel->GetOldestDataTime().AsSeconds(EvaluationType);
		const double NewstedSampleTime = ChannelItem.Channel->GetNewestDataTime().AsSeconds(EvaluationType);
		const double EvaluationTime = GetEvaluationTime(EvaluationType);
		const double OffsettedEvaluationTime = EvaluationTime - EvaluationOffset;
		const bool bIsInRange = (FMath::IsNearlyEqual(OffsettedEvaluationTime, OldestSampleTime) || OffsettedEvaluationTime >= OldestSampleTime) && (FMath::IsNearlyEqual(OffsettedEvaluationTime, NewstedSampleTime) || OffsettedEvaluationTime <= NewstedSampleTime);
		
		NewState = bIsInRange ? ETimedDataMonitorEvaluationState::InsideRange : ETimedDataMonitorEvaluationState::OutsideRange;
	}
	
	if (NewState != ChannelItem.CachedEvaluationState)
	{
		IStageDataProvider::SendMessage<FTimedDataMonitorChannelEvaluationStateEvent>(EStageMessageFlags::Reliable, NewState, Input->GetDisplayName().ToString(), ChannelItem.Channel->GetDisplayName().ToString());
	}

	ChannelItem.CachedEvaluationState = NewState;
}

void UTimedDataMonitorSubsystem::UpdateEvaluationStatistics()
{
	BuildSourcesListIfNeeded();

	for (TPair<FTimedDataMonitorChannelIdentifier, FTimeDataChannelItem>& Item : ChannelMap)
	{
		if (Item.Value.bEnabled)
		{
			FTimedDataInputEvaluationData Data;
			GetChannelLastEvaluationDataStat(Item.Key, Data);

			//Update cached settings to accelerate response of mean tracker
			const ITimedDataInput* Input = InputMap[Item.Value.InputIdentifier].Input;
			const ETimedDataInputEvaluationType EvaluationType = Input->GetEvaluationType();

			const float EvalutionOffset = GetEvaluationTimeOffsetInSeconds(EvaluationType);
			const float TotalTimeOffset = GetInputEvaluationOffsetInSeconds(Item.Value.InputIdentifier) + EvalutionOffset;
			const int32 BufferSize = Input->IsDataBufferSizeControlledByInput() ? Input->GetDataBufferSize() : Item.Value.Channel->GetDataBufferSize();
			Item.Value.Statistics.CacheSettings(EvaluationType, TotalTimeOffset, BufferSize);

			//Update statistics for each channel
			Item.Value.Statistics.Update(Data.DistanceToOldestSampleSeconds, Data.DistanceToNewestSampleSeconds);

			AddStatisticLogEntry(Item);
		}
	}
}

void UTimedDataMonitorSubsystem::UpdateStatFileLoggingState()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bHasStatFileLoggingStarted)
	{
		if (!bIsTimedDataStatFileLoggingStarted)
		{
			const FDateTime NowTime;
			const FString CurrentTimeString = FDateTime::Now().ToString();
			const FString OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("TimedDataMonitor"));

			//Dump each channel to a file
			for (TPair<FTimedDataMonitorChannelIdentifier, FChannelStatisticLogging>& Item : StatLoggingMap)
			{
				const FString Filename = FString::Printf(TEXT("%s_%s.csv"), *CurrentTimeString, *Item.Value.ChannelName);
				const FString FullyQualifiedFileNameFilename = FPaths::Combine(OutputDirectory, Filename);
				FFileHelper::SaveStringArrayToFile(Item.Value.Entries, *FullyQualifiedFileNameFilename);
			}

			bHasStatFileLoggingStarted = false;
		}
	}
	else
	{
		if (bIsTimedDataStatFileLoggingStarted)
		{
			bHasStatFileLoggingStarted = true;
			StatLoggingMap.Empty();
		}
	}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

void UTimedDataMonitorSubsystem::AddStatisticLogEntry(const TPair<FTimedDataMonitorChannelIdentifier, FTimeDataChannelItem>& ChannelEntry)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bHasStatFileLoggingStarted)
	{
		const ITimedDataInput* Input = InputMap[ChannelEntry.Value.InputIdentifier].Input;
		check(Input);
		const ETimedDataInputEvaluationType EvaluationType = Input->GetEvaluationType();
		
		FTimedDataInputEvaluationData Data;
		GetChannelLastEvaluationDataStat(ChannelEntry.Key, Data);

		FChannelStatisticLogging& ChannelStats = StatLoggingMap.FindOrAdd(ChannelEntry.Key);
		if (ChannelStats.Entries.Num() <= 0)
		{
			ChannelStats.ChannelName = ChannelMap[ChannelEntry.Key].Channel->GetDisplayName().ToString();
			const FString Header = TEXT("SampleCount, EvaluationTime,OldestSampleTime, OldestDistance, OldestDistanceMean, OldestDistanceVariance, NewestSampleTime, NewestDistance, NewestDistanceMean, NewestDistanceVariance");
			ChannelStats.Entries.Emplace(Header);
		}

		FString NewEntry = FString::Printf(TEXT("%d,%0.8f,%0.8f,%0.8f,%0.8f,%0.8f,%0.8f,%0.8f,%0.8f,%0.8f")
			, ChannelEntry.Value.Statistics.NewestSampleDistanceTracker.SampleCount
			, GetEvaluationTime(EvaluationType)
			, ChannelEntry.Value.Channel->GetOldestDataTime().AsSeconds(EvaluationType)
			, Data.DistanceToOldestSampleSeconds
			, ChannelEntry.Value.Statistics.OldestSampleDistanceTracker.CurrentMean
			, ChannelEntry.Value.Statistics.OldestSampleDistanceTracker.CurrentVariance
			, ChannelEntry.Value.Channel->GetNewestDataTime().AsSeconds(EvaluationType)
			, Data.DistanceToNewestSampleSeconds
			, ChannelEntry.Value.Statistics.NewestSampleDistanceTracker.CurrentMean
			, ChannelEntry.Value.Statistics.NewestSampleDistanceTracker.CurrentVariance
		);
		ChannelStats.Entries.Emplace(MoveTemp(NewEntry));
	}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

void FTimedDataChannelEvaluationStatistics::CacheSettings(ETimedDataInputEvaluationType EvaluationType, float TimeOffset, int32 BufferSize)
{
	if (EvaluationType != CachedEvaluationType)
	{
		OldestSampleDistanceTracker.Reset();
		NewestSampleDistanceTracker.Reset();
	}

	CachedEvaluationType = EvaluationType;

	//Update next tick offset to feedforward our mean/variance tracker
	NextTickOffset = TimeOffset - CachedOffset;
	
	CachedOffset = TimeOffset;
	CachedBufferSize = BufferSize;
}

void FTimedDataChannelEvaluationStatistics::Update(float DistanceToOldest, float DistanceToNewest)
{
	//Update each stats tracker using the current offset. This only lasts one frame to anticipate stats movement
	OldestSampleDistanceTracker.Update(DistanceToOldest, -NextTickOffset);
	NewestSampleDistanceTracker.Update(DistanceToNewest, NextTickOffset);

	NextTickOffset = 0.0f;
}

void FTimedDataChannelEvaluationStatistics::Reset()
{
	OldestSampleDistanceTracker.Reset();
	NewestSampleDistanceTracker.Reset();
}

void FExponentialMeanVarianceTracker::Reset()
{
	SampleCount = 0;
	CurrentMean = 0.0f;
	CurrentVariance = 0.0f;
	CurrentSTD = 0.0f;
	Alpha = CVarTimedDataStatisticsWeight.GetValueOnGameThread();
}

void FExponentialMeanVarianceTracker::Update(float NewValue, float MeanOffset)
{
	//Exponential moving average computation based on https://en.wikipedia.org/wiki/Moving_average#Exponential_moving_average

	if (SampleCount > 0)
	{
		const float Beta = 1.0f - Alpha;
		const float BetaMean = Beta * (MeanOffset + CurrentMean);
		const float MeanAddition = NewValue * Alpha;
		const float VarianceIncrement = Alpha * FMath::Square(NewValue - (CurrentMean + MeanOffset));
		CurrentMean = BetaMean + MeanAddition;
		CurrentVariance = Beta * (CurrentVariance + VarianceIncrement);
	}
	else
	{
		//When starting exponential count, start at current Value to get better initial response
		CurrentMean = NewValue;
		CurrentVariance = 0.0f;
	}

	LastValue = NewValue;

	CurrentSTD = FMath::Sqrt(CurrentVariance);
	++SampleCount;
}


#undef LOCTEXT_NAMESPACE


