// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/EngineBaseTypes.h"
#include "ITimedDataInput.h"
#include "TimedDataMonitorTypes.h"
#include "UObject/ObjectMacros.h"
#include "Containers/Ticker.h"

#include "TimedDataMonitorCalibration.generated.h"

#define UE_API TIMEDDATAMONITOR_API

UENUM()
enum class ETimedDataMonitorCalibrationReturnCode : uint8
{
	/** Success. The values were synchronized. */
	Succeeded,
	/** Failed. The timecode provider doesn't have a proper timecode value. */
	Failed_NoTimecode,
	/** Failed. At least one input is unresponsive. */
	Failed_UnresponsiveInput,
	/** Failed. At least one input has an evaluation type that is not timecode. */
	Failed_InvalidEvaluationType,
	/** Failed. At least one input doesn't have a defined frame rate. */
	Failed_InvalidFrameRate,
	/** Failed. At least one input doesn't have data buffered. */
	Failed_NoDataBuffered,
	/** Failed. A resize was requested but it was not able to do so. */
	Failed_BufferCouldNotBeResize,
	/** Failed. The calibration was manually reset. */
	Failed_Reset,
	/** Retry. The buffer size is correct but they do not contain enough data to calibrate. */
	Retry_NotEnoughData,
	/** Retry. No interval could be found. Increase the buffer size. */
	Retry_IncreaseBufferSize,
};


USTRUCT(BlueprintType)
struct FTimedDataMonitorCalibrationResult
{
	GENERATED_BODY();

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Result")
	ETimedDataMonitorCalibrationReturnCode ReturnCode = ETimedDataMonitorCalibrationReturnCode::Succeeded;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Result")
	TArray<FTimedDataMonitorInputIdentifier> FailureInputIdentifiers;
};


USTRUCT(BlueprintType)
struct FTimedDataMonitorCalibrationParameters
{
	GENERATED_BODY();

	/** When needed, how many retry is allowed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (ClampMin = 0, ClampMax = 100))
	int32 NumberOfRetries = 4;

	/** If no calibration is possible, are we allowed to increase the size of the buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bBufferResizeAllowed = true;
	
	/** When resizing buffer, do we allow shrinking them. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bBufferShrinkAllowed = false;

	/** When resizing buffer, failed the calibration if a buffer couldn't be resize. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bFailedIfBufferCantBeResize = false;

	/** When calibrating, ensure that the evaluation is included inside the STD. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bUseStandardDeviation = true;

	/** When using STD, how many should we use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (EditCondition = "bUseStandardDeviation", ClampMin = 0, ClampMax = 5))
	int32 NumberOfStandardDeviation = 3;

	/** Before calibration, allow to reset the statistics. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (EditCondition = "bUseStandardDeviation"))
	bool bResetStatisticsBeforeUsingStandardDeviation = true;

	/** Before calibration, allow to reset the statistics. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (EditCondition = "bUseStandardDeviation"))
	float AmountOfSecondsToWaitAfterStatisticReset = 2.f;
};


UENUM()
enum class ETimedDataMonitorTimeCorrectionReturnCode : uint8
{
	/** Success. The values were synchronized. */
	Succeeded,
	/** Failed. The provided input doesn't exist. */
	Failed_InvalidInput,
	/** Failed. The timecode provider was not existing or not synchronized. */
	Failed_NoTimecode,
	/** Failed. At least one channel is unresponsive. */
	Failed_UnresponsiveInput,
	/** Failed. The channel doesn't have any data in it's buffer to synchronized with. */
	Failed_NoDataBuffered,
	/** Failed. A resize was requested but it was not able to do so. */
	Failed_BufferCouldNotBeResize,
	/** Retry. The buffer size is correct but they do not contain enough data to to the time correction. */
	Retry_NotEnoughData,
	/** Retry. No interval could be found. Increase the buffer size. */
	Retry_IncreaseBufferSize,
};

USTRUCT(BlueprintType)
struct FTimedDataMonitorTimeCorrectionResult
{
	GENERATED_BODY();

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Result")
	ETimedDataMonitorTimeCorrectionReturnCode ReturnCode = ETimedDataMonitorTimeCorrectionReturnCode::Succeeded;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Result")
	TArray<FTimedDataMonitorChannelIdentifier> FailureChannelIdentifiers;
};


USTRUCT(BlueprintType)
struct FTimedDataMonitorTimeCorrectionParameters
{
	GENERATED_BODY();

	/** If no calibration is possible, are we allowed to increase the size of the buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bBufferResizeAllowed = true;
	
	/** When resizing buffer, do we allow shrinking them. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bBufferShrinkAllowed = false;

	/** When resizing buffer, failed the calibration if a buffer couldn't be resize. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters")
	bool bFailedIfBufferCantBeResize = false;

	/** When calibrating, ensure that the evaluation is included inside the STD. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (InlineEditConditionToggle = true))
	bool bUseStandardDeviation = true;

	/** When using STD, how many should we use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration Parameters", meta = (EditCondition = "bUseStandardDeviation", ClampMin = 0, ClampMax = 5))
	int32 NumberOfStandardDeviation = 3;
};

/**
 * 
 */
class FTimedDataMonitorCalibration
{
public:
	DECLARE_DELEGATE_OneParam(FOnCalibrationCompletedSignature, FTimedDataMonitorCalibrationResult);

public:
	UE_API ~FTimedDataMonitorCalibration();

	UE_API void Reset();
	UE_API bool IsRunning() const;
	UE_API void CalibrateWithTimecode(const FTimedDataMonitorCalibrationParameters& Parameters, FOnCalibrationCompletedSignature OnCompleted);

	UE_API static FTimedDataMonitorCalibrationResult CalibrateWithTimecode(const FTimedDataMonitorCalibrationParameters& CalibrationParameters);
	UE_API static FTimedDataMonitorTimeCorrectionResult ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& Identifier, const FTimedDataMonitorTimeCorrectionParameters& TimeCorrectionParameters);

private:
	UE_API bool RunCalibrateWithTimecode(float);
	UE_API bool RunCalibrateWithTimecode_CheckForReset(float);

	int32 NumberOfRetry = 0;
	FTimedDataMonitorCalibrationParameters CalibrationParameters;
	FTSTicker::FDelegateHandle TickerHandle;
	FOnCalibrationCompletedSignature OnCalibrationCompleted;
};

#undef UE_API