// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/LatentActionManager.h"
#include "ITimedDataInput.h"
#include "StageMessages.h"
#include "Subsystems/EngineSubsystem.h"
#include "Tickable.h"
#include "TimedDataMonitorCalibration.h"
#include "TimedDataMonitorTypes.h"

#include "TimedDataMonitorSubsystem.generated.h"


UENUM()
enum class ETimedDataMonitorInputEnabled : uint8
{
	Disabled,
	Enabled,
	MultipleValues,
};


UENUM()
enum class ETimedDataMonitorEvaluationState : uint8
{
	NoSample = 0,
	OutsideRange = 1,
	InsideRange = 2,
	Disabled = 3,
};

USTRUCT(meta = (DisplayName = "TimedSourcesConnectionEvent"))
struct FTimedDataMonitorChannelConnectionStateEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FTimedDataMonitorChannelConnectionStateEvent() = default;
	FTimedDataMonitorChannelConnectionStateEvent(ETimedDataInputState InNewState, const FString& InInputName, const FString& InChannelName)
		: NewState(InNewState)
		, InputName(InInputName)
		, ChannelName(InChannelName)
		{}

	virtual FString ToString() const;
	
	/** New state of the channel */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	ETimedDataInputState NewState = ETimedDataInputState::Connected;

	/** Input owning that channel */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	FString InputName;

	/** Channel which had a state change */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	FString ChannelName;
};

USTRUCT(meta = (DisplayName = "TimedSourcesEvaluationEvent"))
struct FTimedDataMonitorChannelEvaluationStateEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FTimedDataMonitorChannelEvaluationStateEvent() = default;
	FTimedDataMonitorChannelEvaluationStateEvent(ETimedDataMonitorEvaluationState InNewState, const FString& InInputName, const FString& InChannelName)
		: NewState(InNewState)
		, InputName(InInputName)
		, ChannelName(InChannelName)
	{}

	virtual FString ToString() const;

	/** New state of the channel */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	ETimedDataMonitorEvaluationState NewState = ETimedDataMonitorEvaluationState::NoSample;

	/** Input owning that channel */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	FString InputName;

	/** Channel which had a state change */
	UPROPERTY(VisibleAnywhere, Category = "Timed Data State")
	FString ChannelName;
};

/**
 * Exponential running mean calculator. Gives better result than incremental running mean when parameters change
 */
struct FExponentialMeanVarianceTracker
{
	FExponentialMeanVarianceTracker()
	{
		Reset();
	}

	void Reset();
	void Update(float NewValue, float MeanOffset);

	/** Current number of samples used for the running average and variance */
	int32 SampleCount = 0;
	
	/** Last sample value used for statistics */
	float LastValue = 0.0f;
	
	/** Current running mean */
	float CurrentMean = 0.0f;
	
	/** Current running variance */
	float CurrentVariance = 0.0f;
	
	/** Current standard deviation */
	float CurrentSTD = 0.0f;
	
	/** Weight given to new samples. A bigger value means new data will weight more in the calculation. Won't filter if data oscillates a lot. */
	float Alpha = 0.0f;
};

/**
 * Structure to facilitate calculating running mean and variance of evaluation distance to buffered samples for a channel
 */
struct FTimedDataChannelEvaluationStatistics
{
	void CacheSettings(ETimedDataInputEvaluationType EvaluationType, float TimeOffset, int32 BufferSize);
	void Update(float DistanceToOldest, float DistanceToNewest);
	void Reset();
	
	FExponentialMeanVarianceTracker NewestSampleDistanceTracker;
	FExponentialMeanVarianceTracker OldestSampleDistanceTracker;

	/** Last evaluation type of our input */
	ETimedDataInputEvaluationType CachedEvaluationType;

	/** Last Offset associated with this channel */
	float CachedOffset = 0.0f;

	/** Last buffer size associated with this channel */
	int32 CachedBufferSize = 0;

	/** Offset to be applied for the next tick. Used to feedforward mean tracker */
	float NextTickOffset = 0.0f;
};


DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTimedDataIdentifierListChangedSignature);


//~ Can be access via GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>()
/**
 * 
 */
UCLASS()
class TIMEDDATAMONITOR_API UTimedDataMonitorSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

private:

	struct FTimeDataInputItem
	{
		ITimedDataInput* Input = nullptr;
		TArray<FTimedDataMonitorChannelIdentifier> ChannelIdentifiers;
		ETimedDataInputState CachedConnectionState;
		ETimedDataMonitorEvaluationState CachedEvaluationState;

	public:
		void ResetValue();
	};

	struct FTimeDataChannelItem
	{
		ITimedDataInputChannel* Channel = nullptr;
		bool bEnabled = true;
		FTimedDataMonitorInputIdentifier InputIdentifier;
		FTimedDataChannelEvaluationStatistics Statistics;
		ETimedDataInputState CachedConnectionState;
		ETimedDataMonitorEvaluationState CachedEvaluationState;

	public:
		void ResetValue();
	};

public:
	//~ Begin USubsystem implementation
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem implementation

	/** Used to cache global (Engine TimecodeProvider) frame delay for the current frame. */
	void BeginFrameCallback();

	/** Used to update statistics once all TimedData are processed. i.e. MediaIO is processed after tickables. */
	void EndFrameCallback();

public:
	/** Delegate of when an element is added or removed. */
	UPROPERTY(BlueprintAssignable, Category = "Timed Data Monitor", meta=(DisplayName="OnSourceIdentifierListChanged"))
	FTimedDataIdentifierListChangedSignature OnIdentifierListChanged_Dynamic;

public:
	/** Delegate of when an element is added or removed. */
	FSimpleMulticastDelegate& OnIdentifierListChanged() { return OnIdentifierListChanged_Delegate; }

	/** Get the interface for a specific input identifier. */
	ITimedDataInput* GetTimedDataInput(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get the interface for a specific channel identifier. */
	ITimedDataInputChannel* GetTimedDataChannel(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get offset applied to global evaluation time. Only works when a Timecode Provider is used */
	float GetEvaluationTimeOffsetInSeconds(ETimedDataInputEvaluationType EvaluationType);

	/** Get the current evaluation time. */
	static double GetEvaluationTime(ETimedDataInputEvaluationType EvaluationType);

public:
	/** Get the list of all the inputs. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	TArray<FTimedDataMonitorInputIdentifier> GetAllInputs();

	/** Get the list of all the channels. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	TArray<FTimedDataMonitorChannelIdentifier> GetAllChannels();

	/** Get the list of all the channels that are enabled. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	TArray<FTimedDataMonitorChannelIdentifier> GetAllEnabledChannels();

	/** Change the Timecode Provider offset to align all inputs and channels. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor", meta = (DisplayName = "Calibrate", Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	void CalibrateLatent(UObject* WorldContextObject, FLatentActionInfo LatentInfo, const FTimedDataMonitorCalibrationParameters& CalibrationParameters, FTimedDataMonitorCalibrationResult& Result);

	/** Assume all data samples were produce at the same time and align them with the current platform's time */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	FTimedDataMonitorTimeCorrectionResult ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& Identifier, const FTimedDataMonitorTimeCorrectionParameters& TimeCorrectionParameters);

	/** Reset the stat of all the inputs. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	void ResetAllBufferStats();

	/** Get the worst evaluation state of all the inputs. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor")
	ETimedDataMonitorEvaluationState GetEvaluationState();

	/** Return true if the identifier is a valid input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	bool DoesInputExist(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Is the input enabled in the monitor. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	ETimedDataMonitorInputEnabled GetInputEnabled(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Set all channels for the input enabled in the monitor. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	void SetInputEnabled(const FTimedDataMonitorInputIdentifier& Identifier, bool bInEnabled);

	/** Return the display name of an input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	FText GetInputDisplayName(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Return the list of all channels that are part of the input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	TArray<FTimedDataMonitorChannelIdentifier> GetInputChannels(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get how the input is evaluated type. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	ETimedDataInputEvaluationType GetInputEvaluationType(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Set how the input is evaluated type. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	void SetInputEvaluationType(const FTimedDataMonitorInputIdentifier& Identifier, ETimedDataInputEvaluationType Evaluation);

	/** Get the offset in seconds or frames (see GetEvaluationType) used at evaluation. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationOffsetInSeconds(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Set the offset in seconds or frames (see GetEvaluationType) used at evaluation. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	void SetInputEvaluationOffsetInSeconds(const FTimedDataMonitorInputIdentifier& Identifier, float Seconds);

	/** Get the offset in frames (see GetEvaluationType) used at evaluation. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationOffsetInFrames(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Set the offset in frames (see GetEvaluationType) used at evaluation. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	void SetInputEvaluationOffsetInFrames(const FTimedDataMonitorInputIdentifier& Identifier, float Frames);

	/** Get the frame rate at which the samples is produce. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	FFrameRate GetInputFrameRate(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get the oldest sample time of all the channel in this input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	FTimedDataChannelSampleTime GetInputOldestDataTime(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get the latest sample time of all the channel in this input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	FTimedDataChannelSampleTime GetInputNewestDataTime(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Does the channel support a different buffer size than it's input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	bool IsDataBufferSizeControlledByInput(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get the size of the buffer used by the input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	int32 GetInputDataBufferSize(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Set the size of the buffer used by the input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	void SetInputDataBufferSize(const FTimedDataMonitorInputIdentifier& Identifier, int32 BufferSize);

	/** Get the worst state of all the channels of that input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	ETimedDataInputState GetInputConnectionState(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Get the worst evaluation state of all the channels of that input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	ETimedDataMonitorEvaluationState GetInputEvaluationState(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Returns the max average distance, in seconds, between evaluation time and newest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationDistanceToNewestSampleMean(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Returns the min average distance, in seconds, between evaluation time and oldest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationDistanceToOldestSampleMean(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Returns the standard deviation of the distance, in seconds, between evaluation time and newest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationDistanceToNewestSampleStandardDeviation(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Returns the standard deviation of the distance, in seconds, between evaluation time and oldest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Input")
	float GetInputEvaluationDistanceToOldestSampleStandardDeviation(const FTimedDataMonitorInputIdentifier& Identifier);

	/** Return true if the identifier is a valid channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	bool DoesChannelExist(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Is the channel enabled in the monitor. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	bool IsChannelEnabled(const FTimedDataMonitorChannelIdentifier& Identifier);

	/**
	 * Enable or disable an input from the monitor.
	 * The input will still be evaluated but stats will not be tracked and the will not be used for calibration.
	 */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	void SetChannelEnabled(const FTimedDataMonitorChannelIdentifier& Identifier, bool bEnabled);

	/** Return the input of this channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	FTimedDataMonitorInputIdentifier GetChannelInput(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Return the display name of an input. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	FText GetChannelDisplayName(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the state the channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	ETimedDataInputState GetChannelConnectionState (const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the evaluation state of the channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	ETimedDataMonitorEvaluationState GetChannelEvaluationState(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the channel oldest sample time. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	FTimedDataChannelSampleTime GetChannelOldestDataTime(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the channel latest sample time. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	FTimedDataChannelSampleTime GetChannelNewestDataTime(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the sample times for every frame in the channel */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	TArray<FTimedDataChannelSampleTime> GetChannelFrameDataTimes(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Get the number of data samples available. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	int32 GetChannelNumberOfSamples(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** If the channel does support it, get the current maximum sample count of channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	int32 GetChannelDataBufferSize(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** If the channel does support it, set the maximum sample count of the channel. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	void SetChannelDataBufferSize(const FTimedDataMonitorChannelIdentifier& Identifier, int32 BufferSize);

	/** Returns the number of buffer underflows detected by that input since the last reset. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	int32 GetChannelBufferUnderflowStat(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Returns the number of buffer overflows detected by that input since the last reset. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	int32 GetChannelBufferOverflowStat(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Returns the number of frames dropped by that input since the last reset. */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	int32 GetChannelFrameDroppedStat(const FTimedDataMonitorChannelIdentifier& Identifier);
	
	/** 
	 * Retrieves information about last evaluation 
	 * Returns true if identifier was found
	 */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	void GetChannelLastEvaluationDataStat(const FTimedDataMonitorChannelIdentifier& Identifier, FTimedDataInputEvaluationData& Result);

	/** Returns the average distance, in seconds, between evaluation time and newest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	float GetChannelEvaluationDistanceToNewestSampleMean(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Returns the average distance, in seconds, between evaluation time and oldest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	float GetChannelEvaluationDistanceToOldestSampleMean(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Returns the standard deviation of the distance, in seconds, between evaluation time and newest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	float GetChannelEvaluationDistanceToNewestSampleStandardDeviation(const FTimedDataMonitorChannelIdentifier& Identifier);

	/** Returns the standard deviation of the distance, in seconds, between evaluation time and oldest sample */
	UFUNCTION(BlueprintCallable, Category = "Timed Data Monitor|Channel")
	float GetChannelEvaluationDistanceToOldestSampleStandardDeviation(const FTimedDataMonitorChannelIdentifier& Identifier);

private:
	void BuildSourcesListIfNeeded();

	void OnTimedDataSourceCollectionChanged();

	/** Caches state with regards to evaluation of registered inputs */
	void CacheEvaluationState();

	/** Caches connection state of each inputs */
	void CacheConnectionState();

	/** Caches the current evaluation state for a channel */
	void CacheChannelEvaluationState(FTimeDataChannelItem& Identifier);

	/** Update internal statistics for each enabled channel */
	void UpdateEvaluationStatistics();

	/** Starts logging stats for each channel to be dumped in a file */
	void UpdateStatFileLoggingState();

	/** Adds a log entry for a given channel */
	void AddStatisticLogEntry(const TPair<FTimedDataMonitorChannelIdentifier, FTimeDataChannelItem>& ChannelEntry);

private:
	bool bRequestSourceListRebuilt = false;
	TMap<FTimedDataMonitorInputIdentifier, FTimeDataInputItem> InputMap;
	TMap<FTimedDataMonitorChannelIdentifier, FTimeDataChannelItem> ChannelMap;
	FSimpleMulticastDelegate OnIdentifierListChanged_Delegate;
	float CachedTimecodeProviderFrameDelayInSeconds = 0.0f;

	/** Cached system state for evaluation and connection based on each input / channel and source that caused last change */
	ETimedDataMonitorEvaluationState CachedEvaluationState = ETimedDataMonitorEvaluationState::Disabled;
	ETimedDataInputState CachedConnectionState = ETimedDataInputState::Disconnected;


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	struct FChannelStatisticLogging
	{
		FString ChannelName;
		TArray<FString> Entries;
	};

	bool bHasStatFileLoggingStarted = false;
	TMap<FTimedDataMonitorChannelIdentifier, FChannelStatisticLogging> StatLoggingMap;
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
};
