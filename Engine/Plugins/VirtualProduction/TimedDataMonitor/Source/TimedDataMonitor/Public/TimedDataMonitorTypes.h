// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "TimedDataMonitorTypes.generated.h"

#define UE_API TIMEDDATAMONITOR_API


USTRUCT(MinimalAPI, BlueprintType)
struct FTimedDataMonitorInputIdentifier
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FGuid Identifier;

public:
	UE_API static FTimedDataMonitorInputIdentifier NewIdentifier();

	bool IsValid() const { return Identifier.IsValid(); }

	bool operator== (const FTimedDataMonitorInputIdentifier& Other) const { return Identifier == Other.Identifier; }
	bool operator!= (const FTimedDataMonitorInputIdentifier& Other) const { return Identifier != Other.Identifier; }
	bool operator< (const FTimedDataMonitorInputIdentifier& Other) const { return Identifier < Other.Identifier; }

	friend uint32 GetTypeHash(const FTimedDataMonitorInputIdentifier& InIdentifier) { return GetTypeHash(InIdentifier.Identifier); }
};


USTRUCT(MinimalAPI, BlueprintType)
struct FTimedDataMonitorChannelIdentifier
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FGuid Identifier;

public:
	UE_API static FTimedDataMonitorChannelIdentifier NewIdentifier();

	bool IsValid() const { return Identifier.IsValid(); }

	bool operator== (const FTimedDataMonitorChannelIdentifier& Other) const { return Identifier == Other.Identifier; }
	bool operator!= (const FTimedDataMonitorChannelIdentifier& Other) const { return Identifier != Other.Identifier; }
	bool operator< (const FTimedDataMonitorChannelIdentifier& Other) const { return Identifier < Other.Identifier; }

	friend uint32 GetTypeHash(const FTimedDataMonitorChannelIdentifier& InIdentifier) { return GetTypeHash(InIdentifier.Identifier); }
};

#undef UE_API