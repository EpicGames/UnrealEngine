// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "TimedDataMonitorEditorSettings.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class SFrameTime : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SFrameTime)
			: _FillImage(FAppStyle::GetBrush(TEXT("WhiteBrush")))
			, _BackgroundImage(&FCoreStyle::Get().GetWidgetStyle<FProgressBarStyle>("ProgressBar").BackgroundImage)
		{
		}

		SLATE_ARGUMENT(TOptional<double>, RefreshRate)
		SLATE_ARGUMENT(const FSlateBrush*, FillImage)
		SLATE_ARGUMENT(const FSlateBrush*, BackgroundImage)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		RefreshRate = InArgs._RefreshRate;
		FillImage = InArgs._FillImage;
		BackgroundImage = InArgs._BackgroundImage;
	}
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (InCurrentTime - LastUpdateTime > RefreshRate.Get(GetDefault<UTimedDataMonitorEditorSettings>()->RefreshRate))
		{
			LastUpdateTime = InCurrentTime;

			const float MaxTickRate = GEngine->GetMaxTickRate(0.001f, false);
			const float TargetFPS = GEngine->bUseFixedFrameRate ? GEngine->FixedFrameRate : MaxTickRate;

			if (FApp::GetIdleTime() == 0 && GEngine->bUseFixedFrameRate && TargetFPS != 0)
			{
				CachedFPSFraction = InDeltaTime / (1 / TargetFPS);
			}
			else
			{
				CachedFPSFraction = FApp::GetDeltaTime() > 0.0 ? (1.f - float(FApp::GetIdleTime() / FApp::GetDeltaTime())) : 1.f;
			}
		}

	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		// Used to track the layer ID we will return.
		int32 RetLayerId = LayerId;
		bool bEnabled = ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			BackgroundImage,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * BackgroundImage->GetTint(InWidgetStyle)
		);

		const double WarnThreshold = GetDefault<UTimedDataMonitorEditorSettings>()->FrameTimeWarnThreshold;

		FSlateColor FillColorAndOpacity = FLinearColor::Green;
		if (CachedFPSFraction >= WarnThreshold && CachedFPSFraction < 1.0)
		{
			FillColorAndOpacity = FLinearColor(FColor::Orange);
		}
		else if (CachedFPSFraction >= 1.0)
		{
			FillColorAndOpacity = FLinearColor(FColor::Red);
		}

		const FLinearColor FillColorAndOpacitySRGB(InWidgetStyle.GetColorAndOpacityTint() * FillColorAndOpacity.GetColor(InWidgetStyle) * FillImage->GetTint(InWidgetStyle));

		float MarqueeAnimOffsetX = 0.0f;
		float MarqueeImageSizeX = 0.0f;
		float MarqueeAnimOffsetY = 0.0f;
		float MarqueeImageSizeY = 0.0f;

		bool bScaleWithFillPerc = true;

		const float XLimit = GetDefault<UTimedDataMonitorEditorSettings>()->TargetFrameTimePercent;
		const float ClampedFraction = FMath::Clamp(CachedFPSFraction * XLimit, 0.0f, 1.0f);

		const FVector2D BorderPadding = FVector2D{ 0.0, 0.0 };
	
		if (PushTransformedClip(OutDrawElements, AllottedGeometry, BorderPadding, FVector2D(0, 0), FSlateRect(0, 0, ClampedFraction, 1)))
		{
			FPaintGeometry PaintRect;
			PaintRect = AllottedGeometry.ToPaintGeometry(
				FVector2D(AllottedGeometry.GetLocalSize().X * ClampedFraction, AllottedGeometry.GetLocalSize().Y),
				FSlateLayoutTransform()
				);

			// Draw fill
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				RetLayerId++,
				PaintRect,
				FillImage,
				DrawEffects,
				FillColorAndOpacitySRGB
			);

			OutDrawElements.PopClip();
		}
		

		RetLayerId++;

		{
			TArray<FVector2D> WarnLinePoints;
			const double XPos = AllottedGeometry.GetLocalSize().X * WarnThreshold * XLimit;
			WarnLinePoints.Add(FVector2D(XPos, 0.f));
			WarnLinePoints.Add(FVector2D(XPos, AllottedGeometry.GetLocalSize().Y));

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				RetLayerId,
				AllottedGeometry.ToPaintGeometry(),
				WarnLinePoints,
				DrawEffects,
				(CachedFPSFraction < WarnThreshold ? FLinearColor(FColor::Orange) : FLinearColor::Black) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}

		{
			TArray<FVector2D> TargetLinePoints;
			const double XPos = AllottedGeometry.GetLocalSize().X * XLimit;
			TargetLinePoints.Add(FVector2D(XPos, 0.f));
			TargetLinePoints.Add(FVector2D(XPos, AllottedGeometry.GetLocalSize().Y));

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				RetLayerId,
				AllottedGeometry.ToPaintGeometry(),
				TargetLinePoints,
				DrawEffects,
				(CachedFPSFraction < 1.f ? FLinearColor::Red : FLinearColor::Black) * InWidgetStyle.GetColorAndOpacityTint(),
				false,
				2.f
			);
		}


		return RetLayerId;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return { 20.0, 12.0 };
	}

	virtual bool ComputeVolatility() const override
	{
		return SLeafWidget::ComputeVolatility();
	}

	/** See attribute Percent */
	void SetPercent(float InPercent)
	{
		if (!Percent.IdenticalTo(InPercent))
		{
			Percent = InPercent;
			Invalidate(EInvalidateWidget::LayoutAndVolatility);
		}
	}

	// Returns false if the clipping zone area is zero in which case we should skip drawing
	bool PushTransformedClip(FSlateWindowElementList& OutDrawElements, const FGeometry& AllottedGeometry, FVector2D InsetPadding, FVector2D ProgressOrigin, FSlateRect Progress) const
	{
		const FSlateRenderTransform& Transform = AllottedGeometry.GetAccumulatedRenderTransform();

		const FVector2D MaxSize = AllottedGeometry.GetLocalSize() - (InsetPadding * 2.0f);

		const FSlateClippingZone ClippingZone(Transform.TransformPoint(InsetPadding + (ProgressOrigin - FVector2D(Progress.Left, Progress.Top)) * MaxSize),
			Transform.TransformPoint(InsetPadding + FVector2D(ProgressOrigin.X + Progress.Right, ProgressOrigin.Y - Progress.Top) * MaxSize),
			Transform.TransformPoint(InsetPadding + FVector2D(ProgressOrigin.X - Progress.Left, ProgressOrigin.Y + Progress.Bottom) * MaxSize),
			Transform.TransformPoint(InsetPadding + (ProgressOrigin + FVector2D(Progress.Right, Progress.Bottom)) * MaxSize));

		if (ClippingZone.HasZeroArea())
		{
			return false;
		}

		OutDrawElements.PushClip(ClippingZone);
		return true;
	}

	float CachedFPSFraction = 0.f;
	TAttribute<float> Percent = 0.f;
	const FSlateBrush* FillImage = nullptr;
	const FSlateBrush* BackgroundImage = nullptr;
	TOptional<double> RefreshRate;
	double LastUpdateTime = 0.0;
};
