// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimedDataGenlock.h"

#include "Editor.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "EditorFontGlyphs.h"
#include "FixedFrameRateCustomTimeStep.h"
#include "SFrameTime.h"

#include "STimedDataMonitorPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "STimedDataGenlock"


void STimedDataGenlock::Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> InOwnerPanel)
{
	OwnerPanel = InOwnerPanel;

	UpdateCachedValue(0.f);

	FSlateFontInfo TimeFont = FCoreStyle::Get().GetFontStyle(TEXT("EmbossedText"));
	TimeFont.Size += 4;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 4.f, 0.f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &STimedDataGenlock::GetStateText)
				.ColorAndOpacity(this, &STimedDataGenlock::GetStateColorAndOpacity)
			]
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
				.Text(this, &STimedDataGenlock::GetCustomTimeStepText)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Bottom)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Padding(4.f)
				.ToolTipText(LOCTEXT("ShowCustomTimeStepSetting_Tip", "Show custom time step provider setting"))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked_Lambda([](){return ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &STimedDataGenlock::ShowCustomTimeStepSetting)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Cogs)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				SNew(SCheckBox)
				.Padding(4.f)
				.ToolTipText(LOCTEXT("ReapplyMenuToolTip", "Reinitialize the current Custom Time Step."))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled(this, &STimedDataGenlock::IsCustomTimeStepEnabled)
				.IsChecked_Lambda([]() {return ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &STimedDataGenlock::ReinitializeCustomTimeStep)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Undo)
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		[
			SNew(SFrameTime)
		]
		+ SVerticalBox::Slot()
		.Padding(0)
		
		.AutoHeight()
		[
			SNew(SGridPanel)
			.FillColumn(0, 0.5f)
			.FillColumn(1, 0.5f)

			+ SGridPanel::Slot(0, 0)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)

				.Text(LOCTEXT("FPSLabel", "FPS: "))
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataGenlock::GetFPSText)
			]
			+ SGridPanel::Slot(0, 1)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeltaTimeLabel", "DeltaTime: "))
			]
			+ SGridPanel::Slot(1, 1)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataGenlock::GetDeltaTimeText)
			]
			+ SGridPanel::Slot(0, 2)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IdleTimeLabel", "Idle Time: "))
			]
			+ SGridPanel::Slot(1, 2)
			.Padding(0.f, 2.f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataGenlock::GetIdleTimeText)
			]
		]
	];
}


void STimedDataGenlock::RequestRefresh()
{
	if (TSharedPtr<STimedDataMonitorPanel> OwnerPanelPin = OwnerPanel.Pin())
	{
		OwnerPanelPin->RequestRefresh();
	}
}

void STimedDataGenlock::UpdateCachedValue(float InDeltaTime)
{
	double IdleTime = FApp::GetIdleTime();

	const float MaxTickRate = GEngine->GetMaxTickRate(0.001f, false);

	float TargetFPS = GEngine->bUseFixedFrameRate ? GEngine->FixedFrameRate : MaxTickRate;
	if (UFixedFrameRateCustomTimeStep* CustomTimeStep = Cast<UFixedFrameRateCustomTimeStep>(GEngine->GetCustomTimeStep()))
	{
		TargetFPS = CustomTimeStep->GetFixedFrameRate().AsDecimal();
	}
	
	const float TargetFrameTime = 1 / TargetFPS;

	FNumberFormattingOptions FPSFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMaximumFractionalDigits(2);
	FNumberFormattingOptions MsFormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMinimumFractionalDigits(3)
		.SetMaximumFractionalDigits(3);

	uint32 IdleFrames = 0;
	if (!FMath::IsNearlyZero(IdleTime))
	{
		IdleFrames = FMath::Abs(FMath::RoundToInt(IdleTime / (TargetFrameTime * TargetFrameTime) ));
	}

	CachedFPSText = FText::AsNumber(1.0/FApp::GetDeltaTime(), &FPSFormattingOptions);
	CachedDeltaTimeText = FText::Format(LOCTEXT("WithMilliSeconds", "{0}ms"), FText::AsNumber(FApp::GetDeltaTime()*1000, &MsFormattingOptions));
	CachedIdleTimeText = FText::Format(LOCTEXT("WithMilliSecondsFrames", "{0}ms ({1} frames)"), FText::AsNumber(IdleTime *1000, &MsFormattingOptions), IdleFrames);

	if (const UEngineCustomTimeStep* CustomTimeStep = GEngine->GetCustomTimeStep())
	{
		CachedState = CustomTimeStep->GetSynchronizationState();
		CachedCustomTimeStepText = FText::FromName(CustomTimeStep->GetFName());
	}
	else
	{
		CachedState = ECustomTimeStepSynchronizationState::Error;
		CachedCustomTimeStepText = LOCTEXT("NoCustomTimeStep", "No custom time step");
	}
}

FText STimedDataGenlock::GetStateText() const
{
	switch (CachedState)
	{
	case ECustomTimeStepSynchronizationState::Synchronized:
		return FEditorFontGlyphs::Clock_O;
	case ECustomTimeStepSynchronizationState::Synchronizing:
		return FEditorFontGlyphs::Hourglass_O;
	case ECustomTimeStepSynchronizationState::Error:
	case ECustomTimeStepSynchronizationState::Closed:
		return FEditorFontGlyphs::Ban;
	}
	return FEditorFontGlyphs::Exclamation;
}


FSlateColor STimedDataGenlock::GetStateColorAndOpacity() const
{
	switch (CachedState)
	{
	case ECustomTimeStepSynchronizationState::Closed:
	case ECustomTimeStepSynchronizationState::Error:
		return FLinearColor::Red;
	case ECustomTimeStepSynchronizationState::Synchronized:
		return FLinearColor::Green;
	case ECustomTimeStepSynchronizationState::Synchronizing:
		return FLinearColor::Yellow;
	}

	return FLinearColor::Red;
}


FText STimedDataGenlock::GetCustomTimeStepText() const
{
	return CachedCustomTimeStepText;
}


bool STimedDataGenlock::IsCustomTimeStepEnabled() const
{
	return GEngine && GEngine->GetCustomTimeStep() != nullptr;
}

void STimedDataGenlock::ShowCustomTimeStepSetting(ECheckBoxState)
{
	if (UEngineCustomTimeStep* CustomTimeStepPtr = GEngine->GetCustomTimeStep())
	{
		if (CustomTimeStepPtr->GetOuter() == GEngine)
		{
			FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Engine", "General");
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CustomTimeStepPtr->GetOuter());
		}
	}
	else
	{
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Engine", "General");
	}
}


void STimedDataGenlock::ReinitializeCustomTimeStep(ECheckBoxState)
{
	if (GEngine)
	{
		GEngine->ReinitializeCustomTimeStep();
	}
}


#undef LOCTEXT_NAMESPACE
