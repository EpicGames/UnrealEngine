// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"


enum class ECustomTimeStepSynchronizationState;

class STimedDataMonitorPanel;
enum class ECheckBoxState : uint8;

class STimedDataGenlock : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimedDataGenlock) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> OwnerPanel);

	void RequestRefresh();
	void UpdateCachedValue(float InDeltaTime);

private:
	FText GetStateText() const;
	FSlateColor GetStateColorAndOpacity() const;
	FText GetCustomTimeStepText() const;

	bool IsCustomTimeStepEnabled() const;
	float GetFPSFraction() const;

	FText GetFPSText() const  { return CachedFPSText; }
	FText GetDeltaTimeText() const  { return CachedDeltaTimeText; }
	FText GetIdleTimeText() const  { return CachedIdleTimeText; }

	void ShowCustomTimeStepSetting(ECheckBoxState);
	void ReinitializeCustomTimeStep(ECheckBoxState);

private:
	TWeakPtr<STimedDataMonitorPanel> OwnerPanel;

	ECustomTimeStepSynchronizationState CachedState;
	FText CachedCustomTimeStepText;
	FText CachedFPSText;
	FText CachedDeltaTimeText;
	FText CachedIdleTimeText;
};
