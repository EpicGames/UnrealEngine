// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimedDataListView.h"

#include "Framework/Application/SlateUser.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "TimedDataMonitorSubsystem.h"
#include "SEnumCombo.h"
#include "STimedDataMonitorPanel.h"
#include "STimedDataNumericEntryBox.h"
#include "STimingDiagramWidget.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


#define LOCTEXT_NAMESPACE "STimedDataListView"


namespace TimedDataListView
{
	const FName HeaderIdName_Enable			= "Enable";
	const FName HeaderIdName_Icon			= "Edit";
	const FName HeaderIdName_EvaluationMode = "EvaluationMode";
	const FName HeaderIdName_Name			= "Name";
	const FName HeaderIdName_Description	= "Description";
	const FName HeaderIdName_TimeCorrection	= "TimeCorrection";
	const FName HeaderIdName_BufferSize		= "BufferSize";
	const FName HeaderIdName_BufferUnder	= "BufferUnder";
	const FName HeaderIdName_BufferOver		= "BufferOver";
	const FName HeaderIdName_FrameDrop		= "FrameDrop";
	const FName HeaderIdName_TimingDiagram	= "TimingDiagram";

	FTimespan FromPlatformSeconds(double InPlatformSeconds)
	{
		const FDateTime NowDateTime = FDateTime::Now();
		const double HighPerformanceClock = FPlatformTime::Seconds();
		const double DateTimeSeconds = (InPlatformSeconds - HighPerformanceClock) + NowDateTime.GetTimeOfDay().GetTotalSeconds();
		return FTimespan::FromSeconds(DateTimeSeconds);
	}
}

/**
 * FTimedDataTableRowData
 */
struct FTimedDataInputTableRowData : TSharedFromThis<FTimedDataInputTableRowData>
{
	FTimedDataInputTableRowData(const FTimedDataMonitorInputIdentifier& InInputId)
		: InputIdentifier(InInputId), bIsInput(true)
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);

		DisplayName = TimedDataMonitorSubsystem->GetInputDisplayName(InputIdentifier);

		const ETimedDataInputEvaluationType InputType = TimedDataMonitorSubsystem->GetInputEvaluationType(InputIdentifier);
		if (InputType == ETimedDataInputEvaluationType::Timecode)
		{
			CachedInputEvaluationOffsetType = ETimedDataInputEvaluationOffsetType::Frames;
		}
		else
		{
			CachedInputEvaluationOffsetType = ETimedDataInputEvaluationOffsetType::Seconds;
		}

		if (const ITimedDataInput* InputTimedData = TimedDataMonitorSubsystem->GetTimedDataInput(InputIdentifier))
		{
			InputIcon = InputTimedData->GetDisplayIcon();
			bSupportsSubFrames = InputTimedData->SupportsSubFrames();
		}
	}

	FTimedDataInputTableRowData(const FTimedDataMonitorChannelIdentifier& InChannelId)
		: ChannelIdentifier(InChannelId), bIsInput(false)
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);

		InputIdentifier = TimedDataMonitorSubsystem->GetChannelInput(ChannelIdentifier);
		DisplayName = TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier);
	}

	void UpdateCachedValue()
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);

		FTimedDataChannelSampleTime NewestDataTime;
		if (bIsInput)
		{
			switch (TimedDataMonitorSubsystem->GetInputEnabled(InputIdentifier))
			{
			case ETimedDataMonitorInputEnabled::Enabled:
				CachedEnabled = ECheckBoxState::Checked;
				break;
			case ETimedDataMonitorInputEnabled::Disabled:
				CachedEnabled = ECheckBoxState::Unchecked;
				break;
			case ETimedDataMonitorInputEnabled::MultipleValues:
			default:
				CachedEnabled = ECheckBoxState::Undetermined;
				break;
			};

			CachedInputEvaluationType = TimedDataMonitorSubsystem->GetInputEvaluationType(InputIdentifier);

			CachedInputEvaluationOffsetSeconds = TimedDataMonitorSubsystem->GetInputEvaluationOffsetInSeconds(InputIdentifier);
			CachedInputEvaluationOffsetFrames = TimedDataMonitorSubsystem->GetInputEvaluationOffsetInFrames(InputIdentifier);
			CachedState = TimedDataMonitorSubsystem->GetInputConnectionState(InputIdentifier);
			CachedBufferSize = TimedDataMonitorSubsystem->GetInputDataBufferSize(InputIdentifier);
			CachedCurrentAmountOfBuffer = 0;
			bControlBufferSize = TimedDataMonitorSubsystem->IsDataBufferSizeControlledByInput(InputIdentifier);
			bCachedCanEditBufferSize = (CachedEnabled == ECheckBoxState::Checked || CachedEnabled == ECheckBoxState::Undetermined) && bControlBufferSize;

			NewestDataTime = TimedDataMonitorSubsystem->GetInputNewestDataTime(InputIdentifier);
			if (NewestDataTime.Timecode.Time == 0)
			{
				NewestDataTime.Timecode.Rate = TimedDataMonitorSubsystem->GetInputFrameRate(InputIdentifier);
			}

			CachedStatsBufferUnderflow = 0;
			CachedStatsBufferOverflow = 0;
			CachedStatsFrameDropped = 0;

			for (FTimedDataInputTableRowDataPtr& Child : InputChildren)
			{
				Child->UpdateCachedValue();

				//Update the group stats here to simplify the queries
				CachedStatsBufferUnderflow = FMath::Max(CachedStatsBufferUnderflow, Child->CachedStatsBufferUnderflow);
				CachedStatsBufferOverflow = FMath::Max(CachedStatsBufferOverflow, Child->CachedStatsBufferOverflow);
				CachedStatsFrameDropped += Child->CachedStatsFrameDropped;
			}
		}
		else
		{
			CachedEnabled = TimedDataMonitorSubsystem->IsChannelEnabled(ChannelIdentifier) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			CachedInputEvaluationType = TimedDataMonitorSubsystem->GetInputEvaluationType(InputIdentifier);
			CachedInputEvaluationOffsetSeconds = 0.f;
			CachedInputEvaluationOffsetFrames = 0.f;
			CachedState = TimedDataMonitorSubsystem->GetChannelConnectionState(ChannelIdentifier);
			CachedBufferSize = TimedDataMonitorSubsystem->GetChannelDataBufferSize(ChannelIdentifier);
			CachedCurrentAmountOfBuffer = TimedDataMonitorSubsystem->GetChannelNumberOfSamples(ChannelIdentifier);
			CachedStatsBufferUnderflow = TimedDataMonitorSubsystem->GetChannelBufferUnderflowStat(ChannelIdentifier);
			CachedStatsBufferOverflow = TimedDataMonitorSubsystem->GetChannelBufferOverflowStat(ChannelIdentifier);
			CachedStatsFrameDropped = TimedDataMonitorSubsystem->GetChannelFrameDroppedStat(ChannelIdentifier);
			bControlBufferSize = !TimedDataMonitorSubsystem->IsDataBufferSizeControlledByInput(InputIdentifier);
			bCachedCanEditBufferSize = (CachedEnabled == ECheckBoxState::Checked || CachedEnabled == ECheckBoxState::Undetermined) && bControlBufferSize;

			NewestDataTime = TimedDataMonitorSubsystem->GetChannelNewestDataTime(ChannelIdentifier);
		}

		if (CachedEnabled == ECheckBoxState::Checked)
		{
			switch (CachedInputEvaluationType)
			{
			case ETimedDataInputEvaluationType::Timecode:
			{
				const FTimecode Timecode = NewestDataTime.Timecode.ToTimecode();

				if (Timecode == FTimecode())
				{ 
					// Timecode isn't valid.
					CachedDescription = LOCTEXT("NoTimecodeLabel", "No Timecode");
				}
				else
				{
					CachedDescription = FText::Format(
						INVTEXT("{0} @ {1}"), 
						FText::FromString(Timecode.ToString()),
						NewestDataTime.Timecode.Rate.ToPrettyText()
					);
				}
			}
			break;
			case ETimedDataInputEvaluationType::PlatformTime:
			{
				FTimespan PlatformSecond = TimedDataListView::FromPlatformSeconds(NewestDataTime.PlatformSecond);
				CachedDescription = FText::FromString(PlatformSecond.ToString());
			}
			break;
			case ETimedDataInputEvaluationType::None:
			default:
				CachedDescription = FText::GetEmpty();
				break;
			}
		}
	}

public:
	FTimedDataMonitorInputIdentifier InputIdentifier;
	FTimedDataMonitorChannelIdentifier ChannelIdentifier;
	bool bIsInput;

	FText DisplayName;
	const FSlateBrush* InputIcon = nullptr;
	TArray<FTimedDataInputTableRowDataPtr> InputChildren;
	bool bSupportsSubFrames = true;

	ECheckBoxState CachedEnabled = ECheckBoxState::Undetermined;
	ETimedDataInputEvaluationType CachedInputEvaluationType = ETimedDataInputEvaluationType::None;
	ETimedDataInputEvaluationOffsetType CachedInputEvaluationOffsetType = ETimedDataInputEvaluationOffsetType::Seconds;
	float CachedInputEvaluationOffsetSeconds = 0.f;
	float CachedInputEvaluationOffsetFrames = 0.f;

	ETimedDataInputState CachedState = ETimedDataInputState::Disconnected;
	FText CachedDescription;
	int32 CachedBufferSize = 0;
	int32 CachedCurrentAmountOfBuffer = 0;
	int32 CachedStatsBufferUnderflow = 0;
	int32 CachedStatsBufferOverflow = 0;
	int32 CachedStatsFrameDropped = 0;
	bool bControlBufferSize = false;
	bool bCachedCanEditBufferSize = false;
};


/**
 * STimedDataInputTableRow
 */
void STimedDataInputTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwerTableView, const TSharedRef<STimedDataInputListView>& InOwnerTreeView)
{
	Item = InArgs._Item;
	OwnerTreeView = InOwnerTreeView;
	check(Item.IsValid());

	Super::FArguments Arg;

	if (Item->bIsInput)
	{
		Arg.Style(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"));
	}
	else
	{
		Arg.Style(FTimedDataMonitorEditorStyle::Get(), "TableView.Child");
	}
	Super::Construct(Arg, InOwerTableView);
}


void STimedDataInputTableRow::UpdateCachedValue()
{
	if (DiagramWidget)
	{
		DiagramWidget->UpdateCachedValue();
	}
}


TSharedRef<SWidget> STimedDataInputTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const FTextBlockStyle* ItemTextBlockStyle = Item->bIsInput
		? &FTimedDataMonitorEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TextBlock.Large")
		: &FTimedDataMonitorEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TextBlock.Regular");

	if (TimedDataListView::HeaderIdName_Enable == ColumnName)
	{
		const FText Tooltip = Item->bIsInput
			? LOCTEXT("ToggleAllChannelsToolTip", "Toggles all channels from this input.")
			: LOCTEXT("ToggleChannelToolTip", "Toggles whether this channel will collect stats and be used when calibrating.");
		return SNew(SCheckBox)
			.Style(FTimedDataMonitorEditorStyle::Get(), "CheckBox.Enable")
			.ToolTipText(Tooltip)
			.IsChecked(this, &STimedDataInputTableRow::GetEnabledCheckState)
			.OnCheckStateChanged(this, &STimedDataInputTableRow::OnEnabledCheckStateChanged);
	}
	if (TimedDataListView::HeaderIdName_Icon == ColumnName)
	{
		if (Item->bIsInput)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6, 0, 0, 0)
				[
					SNew(SExpanderArrow, SharedThis<STimedDataInputTableRow>(this))
					.ShouldDrawWires(false)
					.IndentAmount(12)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(SImage)
					.Image(Item->InputIcon)
				];
		}
		return SNullWidget::NullWidget;
	}
	if (TimedDataListView::HeaderIdName_EvaluationMode == ColumnName)
	{
		if (Item->bIsInput)
		{
			return SNew(SComboButton)
				.ComboButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleComboButton")
				.HAlign(HAlign_Center)
				.OnGetMenuContent(this, &STimedDataInputTableRow::OnEvaluationImageBuildMenu)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(this, &STimedDataInputTableRow::GetEvaluationImage)
				];
		}
		return SNullWidget::NullWidget;
	}
	if (TimedDataListView::HeaderIdName_Name == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(10, 0, 10, 0)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &STimedDataInputTableRow::GetStateGlyphs)
				.ColorAndOpacity(this, &STimedDataInputTableRow::GetStateColorAndOpacity)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Item->DisplayName)
				.TextStyle(ItemTextBlockStyle)
			];
	}
	if (TimedDataListView::HeaderIdName_Description == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &STimedDataInputTableRow::GetDescription)
				.TextStyle(ItemTextBlockStyle)
			];
	}
	if (TimedDataListView::HeaderIdName_TimeCorrection == ColumnName)
	{
		if (Item->bIsInput)
		{
			const TSharedRef<SWidget> Label = SNew(SEnumComboBox, StaticEnum<ETimedDataInputEvaluationOffsetType>())
				.ContentPadding(0.f)
				.CurrentValue_Lambda([this]() { return static_cast<int32>(Item->CachedInputEvaluationOffsetType); })
				.OnEnumSelectionChanged(this, &STimedDataInputTableRow::OnEvaluationOffsetTypeChanged);

			static const FString SecondString = TEXT("seconds");
			static const FString FrameString = TEXT("frames");

			auto MinusButtonTooltipTextLambda = [this]()
			{
				const TCHAR* UnitString = Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds ? *SecondString : *FrameString;
				return FText::Format(LOCTEXT("RushToolTip", "Rush this source (in {0})."), FText::FromStringView(UnitString));
			};

			auto PlusButtonTooltipTextLambda = [this]()
			{
				const TCHAR* UnitString = Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds ? *SecondString : *FrameString;
				return FText::Format(LOCTEXT("DelayToolTip", "Delay this source (in {0})."), FText::FromStringView(UnitString));
			};

			return SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this]()
				{
					bool bUseFloatEntryBox = true;
					if (!Item->bSupportsSubFrames)
					{
						bUseFloatEntryBox = Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds;
					}
					return bUseFloatEntryBox ? 0 : 1;
				})
				+SWidgetSwitcher::Slot()
				[
					SNew(STimedDataNumericEntryBox<double>)
						.TextStyle(ItemTextBlockStyle)
						.Value(this, &STimedDataInputTableRow::GetFloatEvaluationOffset)
						.ComboButton(false)
						.SuffixWidget(Label)
						.MinusButtonToolTipText_Lambda(MinusButtonTooltipTextLambda)
						.PlusButtonToolTipText_Lambda(PlusButtonTooltipTextLambda)
						.OnValueCommitted(this, &STimedDataInputTableRow::SetEvaluationOffset)
				]
				+SWidgetSwitcher::Slot()
				[
					SNew(STimedDataNumericEntryBox<int32>)
						.TextStyle(ItemTextBlockStyle)
						.Value(this, &STimedDataInputTableRow::GetIntEvaluationOffset)
						.ComboButton(false)
						.SuffixWidget(Label)
						.MinusButtonToolTipText_Lambda(MinusButtonTooltipTextLambda)
						.PlusButtonToolTipText_Lambda(PlusButtonTooltipTextLambda)
						.OnValueCommitted(this, &STimedDataInputTableRow::SetEvaluationOffset)
				];
		}
		return SNullWidget::NullWidget;
	}
	if (TimedDataListView::HeaderIdName_BufferSize == ColumnName)
	{
		if (Item->bControlBufferSize)
		{
			return SNew(STimedDataNumericEntryBox<int32>)
				.TextStyle(ItemTextBlockStyle)
				.ToolTipText(LOCTEXT("BufferSize_ToolTip", "Buffer Size."))
				.Value(this, &STimedDataInputTableRow::GetBufferSize)
				.ShowAmount(!Item->bIsInput)
				.Amount(this, &STimedDataInputTableRow::GetCurrentSampleCount)
				.EditLabel(LOCTEXT("BufferSize_EditLable", "Number of buffer: "))
				.OnValueCommitted(this, &STimedDataInputTableRow::SetBufferSize)
				.CanEdit(Item->bControlBufferSize)
				.IsEnabled(this, &STimedDataInputTableRow::CanEditBufferSize);
		}
		else
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(ItemTextBlockStyle)
					.Text(this, &STimedDataInputTableRow::GetBufferSizeText)
				];
		}
	}
	if (TimedDataListView::HeaderIdName_BufferUnder == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataInputTableRow::GetBufferUnderflowCount)
				.TextStyle(ItemTextBlockStyle)
			];
	}
	if (TimedDataListView::HeaderIdName_BufferOver == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataInputTableRow::GetBufferOverflowCount)
				.TextStyle(ItemTextBlockStyle)
			];
	}
	if (TimedDataListView::HeaderIdName_FrameDrop == ColumnName)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(this, &STimedDataInputTableRow::GetFrameDroppedCount)
				.TextStyle(ItemTextBlockStyle)
			];
	}
	if (TimedDataListView::HeaderIdName_TimingDiagram == ColumnName)
	{
		return SAssignNew(DiagramWidget, STimingDiagramWidget, Item->bIsInput)
				.ChannelIdentifier(Item->ChannelIdentifier)
				.InputIdentifier(Item->InputIdentifier)
				.ShowSigma(true)
				.ShowFurther(true);
	}

	return SNullWidget::NullWidget;
}


ECheckBoxState STimedDataInputTableRow::GetEnabledCheckState() const
{
	return Item->CachedEnabled;
}


void STimedDataInputTableRow::OnEnabledCheckStateChanged(ECheckBoxState NewState)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	if (Item->bIsInput)
	{
		TimedDataMonitorSubsystem->SetInputEnabled(Item->InputIdentifier, NewState == ECheckBoxState::Checked);
	}
	else
	{
		TimedDataMonitorSubsystem->SetChannelEnabled(Item->ChannelIdentifier, NewState == ECheckBoxState::Checked);
	}
	OwnerTreeView->MarkSettingsDirty();
	OwnerTreeView->RequestRefresh();
}


FText STimedDataInputTableRow::GetStateGlyphs() const
{
	return (Item->CachedEnabled == ECheckBoxState::Checked) ?  FEditorFontGlyphs::Circle :  FEditorFontGlyphs::Circle_O;
}


FSlateColor STimedDataInputTableRow::GetStateColorAndOpacity() const
{
	if (Item->CachedEnabled != ECheckBoxState::Unchecked)
	{
		switch (Item->CachedState)
		{
		case ETimedDataInputState::Connected:
			return FLinearColor::Green;
		case ETimedDataInputState::Disconnected:
			return FLinearColor::Red;
		case ETimedDataInputState::Unresponsive:
			return FLinearColor::Yellow;
		}
	}

	return FSlateColor::UseForeground();
}


FText STimedDataInputTableRow::GetDescription() const
{
	return Item->CachedDescription;
}


FText STimedDataInputTableRow::GetEvaluationOffsetText() const
{
	if (Item->bIsInput)
	{
		return Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds ? FText::AsNumber(Item->CachedInputEvaluationOffsetSeconds) : FText::AsNumber(Item->CachedInputEvaluationOffsetFrames);
	}
	return FText::GetEmpty();
}


double STimedDataInputTableRow::GetFloatEvaluationOffset() const
{
	const double Result = Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds ? Item->CachedInputEvaluationOffsetSeconds : Item->CachedInputEvaluationOffsetFrames;
	return FMath::RoundToFloat(Result * 10000.0) / 10000.0;
}


int32 STimedDataInputTableRow::GetIntEvaluationOffset() const
{
	return FMath::RoundToInt32(Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds ? Item->CachedInputEvaluationOffsetSeconds : Item->CachedInputEvaluationOffsetFrames);
}


void STimedDataInputTableRow::SetEvaluationOffset(double InValue, ETextCommit::Type CommitType)
{
	if (GetFloatEvaluationOffset() != InValue && Item->bIsInput)
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);

		if (Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Seconds)
		{
			Item->CachedInputEvaluationOffsetSeconds = InValue;
			TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(Item->InputIdentifier, InValue);
		}
		else if (Item->CachedInputEvaluationOffsetType == ETimedDataInputEvaluationOffsetType::Frames)
		{
			Item->CachedInputEvaluationOffsetFrames = InValue;
			TimedDataMonitorSubsystem->SetInputEvaluationOffsetInFrames(Item->InputIdentifier, InValue);
		}
		else
		{
			ensure(false);
		}

		OwnerTreeView->MarkSettingsDirty();
		OwnerTreeView->RequestRefresh();
	}
}


void STimedDataInputTableRow::SetEvaluationOffset(int32 InValue, ETextCommit::Type CommitType)
{
	SetEvaluationOffset(static_cast<float>(InValue), CommitType);
}

int32 STimedDataInputTableRow::GetBufferSize() const
{
	return Item->CachedBufferSize;
}


FText STimedDataInputTableRow::GetBufferSizeText() const
{
	if (Item->bIsInput)
	{
		return FText::GetEmpty();
	}
	else
	{
		return FText::Format(LOCTEXT("ChannelBufferSizeFormat", "{0}/{1}"), Item->CachedCurrentAmountOfBuffer, Item->CachedBufferSize);
	}
}


void STimedDataInputTableRow::SetBufferSize(int32 InValue, ETextCommit::Type InType)
{
	if (GetBufferSize() != InValue)
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);

		if (Item->bIsInput)
		{
			TimedDataMonitorSubsystem->SetInputDataBufferSize(Item->InputIdentifier, InValue);
			Item->CachedBufferSize = TimedDataMonitorSubsystem->GetInputDataBufferSize(Item->InputIdentifier);
		}
		else
		{
			TimedDataMonitorSubsystem->SetChannelDataBufferSize(Item->ChannelIdentifier, InValue);
			Item->CachedBufferSize = TimedDataMonitorSubsystem->GetChannelDataBufferSize(Item->ChannelIdentifier);
		}
		OwnerTreeView->MarkSettingsDirty();
		OwnerTreeView->RequestRefresh();
	}
}


bool STimedDataInputTableRow::CanEditBufferSize() const
{
	return Item->bCachedCanEditBufferSize;
}


int32 STimedDataInputTableRow::GetCurrentSampleCount() const
{
	return Item->CachedCurrentAmountOfBuffer;
}

TSharedRef<SWidget> STimedDataInputTableRow::OnEvaluationImageBuildMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const ETimedDataInputEvaluationType CurrentEvaluationType = Item->CachedInputEvaluationType;

	ETimedDataInputEvaluationType LambdaEvaluationType = ETimedDataInputEvaluationType::Timecode;
	MenuBuilder.AddMenuEntry(
		LOCTEXT("EvaluationTypeTimecodeLabel", "Timecode"),
		LOCTEXT("EvaluationTypeTimecodeTooltip", "Evaluate the input base on the engine's timecode value."),
		FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), FTimedDataMonitorEditorStyle::NAME_TimecodeBrush),
		FUIAction(
			FExecuteAction::CreateSP(this, &STimedDataInputTableRow::SetInputEvaluationType, LambdaEvaluationType),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([CurrentEvaluationType, LambdaEvaluationType]() { return CurrentEvaluationType == LambdaEvaluationType; })
		));

	LambdaEvaluationType = ETimedDataInputEvaluationType::PlatformTime;
	MenuBuilder.AddMenuEntry(
		LOCTEXT("EvaluationTypePlatformTimeLabel", "Platform Time"),
		LOCTEXT("EvaluationTypePlatformTimeTooltip", "Evaluate the input base on the engine's time."),
		FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), FTimedDataMonitorEditorStyle::NAME_PlatformTimeBrush),
		FUIAction(
			FExecuteAction::CreateSP(this, &STimedDataInputTableRow::SetInputEvaluationType, LambdaEvaluationType),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([CurrentEvaluationType, LambdaEvaluationType]() { return CurrentEvaluationType == LambdaEvaluationType; })
		));

	LambdaEvaluationType = ETimedDataInputEvaluationType::None;
	MenuBuilder.AddMenuEntry(
		LOCTEXT("EvaluationTypeNoneLabel", "No synchronization"),
		LOCTEXT("EvaluationTypeNoneTooltip", "Do not create any special evaluation (take the latest sample available)."),
		FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), FTimedDataMonitorEditorStyle::NAME_NoEvaluationBrush),
		FUIAction(
			FExecuteAction::CreateSP(this, &STimedDataInputTableRow::SetInputEvaluationType, LambdaEvaluationType),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([CurrentEvaluationType, LambdaEvaluationType]() { return CurrentEvaluationType == LambdaEvaluationType; })
		));

	return MenuBuilder.MakeWidget();
}


const FSlateBrush* STimedDataInputTableRow::GetEvaluationImage() const
{
	if (Item->CachedInputEvaluationType == ETimedDataInputEvaluationType::Timecode)
	{
		return FTimedDataMonitorEditorStyle::Get().GetBrush(FTimedDataMonitorEditorStyle::NAME_TimecodeBrush);
	}
	if (Item->CachedInputEvaluationType == ETimedDataInputEvaluationType::PlatformTime)
	{
		return FTimedDataMonitorEditorStyle::Get().GetBrush(FTimedDataMonitorEditorStyle::NAME_PlatformTimeBrush);
	}
	return FTimedDataMonitorEditorStyle::Get().GetBrush(FTimedDataMonitorEditorStyle::NAME_NoEvaluationBrush);
}


void STimedDataInputTableRow::SetInputEvaluationType(ETimedDataInputEvaluationType EvaluationType)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	if (Item->bIsInput)
	{
		TimedDataMonitorSubsystem->SetInputEvaluationType(Item->InputIdentifier, EvaluationType);

		OwnerTreeView->MarkSettingsDirty();
		OwnerTreeView->RequestRefresh();
	}
}


FText STimedDataInputTableRow::GetBufferUnderflowCount() const
{
	return FText::AsNumber(Item->CachedStatsBufferUnderflow);
}

FText STimedDataInputTableRow::GetBufferOverflowCount() const
{
	return FText::AsNumber(Item->CachedStatsBufferOverflow);
}

FText STimedDataInputTableRow::GetFrameDroppedCount() const
{
	return FText::AsNumber(Item->CachedStatsFrameDropped);
}

void STimedDataInputTableRow::OnEvaluationOffsetTypeChanged(int32 NewValue, ESelectInfo::Type)
{
	FSlateApplication::Get().ForEachUser([&](FSlateUser& User)
	{
		User.ClearFocus(EFocusCause::SetDirectly);
	});

	Item->CachedInputEvaluationOffsetType = static_cast<ETimedDataInputEvaluationOffsetType>(NewValue);
}

/**
 * STimedDataListView
 */
void STimedDataInputListView::Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> InOwnerPanel)
{
	OwnerPanel = InOwnerPanel;
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);
	TimedDataMonitorSubsystem->OnIdentifierListChanged().AddSP(this, &STimedDataInputListView::RequestRebuildSources);

	Super::Construct
	(
		Super::FArguments()
		.TreeItemsSource(&ListItemsSource)
		.SelectionMode(ESelectionMode::SingleToggle)
		.OnGenerateRow(this, &STimedDataInputListView::OnGenerateRow)
		.OnRowReleased(this, &STimedDataInputListView::ReleaseListViewWidget)
		.OnGetChildren(this, &STimedDataInputListView::GetChildrenForInfo)
		.OnSelectionChanged(this, &STimedDataInputListView::OnSelectionChanged)
		.OnIsSelectableOrNavigable(this, &STimedDataInputListView::OnIsSelectableOrNavigable)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.HighlightParentNodesForSelection(true)
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_Enable)
			.FixedWidth(32)
			.DefaultLabel(FText::GetEmpty())
			[
				SNew(SCheckBox)
				.HAlign(HAlign_Center)
				.IsChecked(this, &STimedDataInputListView::GetAllEnabledCheckState)
				.OnCheckStateChanged(this, &STimedDataInputListView::OnToggleAllEnabledCheckState)
			]

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_Icon)
			.FixedWidth(32)
			.HAlignCell(EHorizontalAlignment::HAlign_Center)
			.VAlignCell(EVerticalAlignment::VAlign_Center)
			.DefaultLabel(FText::GetEmpty())

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_Name)
			.FillWidth(0.33f)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("HeaderName_Name", "Name"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_Description)
			.FillWidth(0.33f)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("HeaderName_Description", "Last Sample Time"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_EvaluationMode)
			.FixedWidth(48)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("HeaderName_EvaluationMode", "Eval."))
			.DefaultTooltip(LOCTEXT("HeaderTooltip_EvaluationMode", "How the input is evaluated"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_TimeCorrection)
			.FixedWidth(170)
			.HAlignCell(EHorizontalAlignment::HAlign_Center)
			.DefaultLabel(LOCTEXT("HeaderName_TimeCorrection", "Time Correction"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_BufferSize)
			.FixedWidth(100)
			.HAlignCell(EHorizontalAlignment::HAlign_Right)
			.DefaultLabel(LOCTEXT("HeaderName_BufferSize", "Buffer Size"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_BufferUnder)
			.FixedWidth(50)
			.HAlignCell(EHorizontalAlignment::HAlign_Right)
			.DefaultLabel(LOCTEXT("HeaderName_BufferUnder", "B.U."))
			.DefaultTooltip(LOCTEXT("HeaderTooltip_BufferUnder", "Number of buffer underflows detected"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_BufferOver)
			.FixedWidth(50)
			.HAlignCell(EHorizontalAlignment::HAlign_Right)
			.DefaultLabel(LOCTEXT("HeaderName_BufferOver", "B.O."))
			.DefaultTooltip(LOCTEXT("HeaderTooltip_BufferOver", "Number of buffer overflows detected"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_FrameDrop)
			.FixedWidth(50)
			.HAlignCell(EHorizontalAlignment::HAlign_Right)
			.DefaultLabel(LOCTEXT("HeaderName_FrameDrop", "F.D."))
			.DefaultTooltip(LOCTEXT("HeaderTooltip_FrameDrop", "Number of frame drop detected"))

			+ SHeaderRow::Column(TimedDataListView::HeaderIdName_TimingDiagram)
			.FillWidth(0.33f)
			.HAlignCell(EHorizontalAlignment::HAlign_Fill)
			.DefaultLabel(LOCTEXT("HeaderName_TimingDiagram", "Timing Diagram"))
		)
	);
}


STimedDataInputListView::~STimedDataInputListView()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	if (TimedDataMonitorSubsystem)
	{
		TimedDataMonitorSubsystem->OnIdentifierListChanged().RemoveAll(this);
	}
}


void STimedDataInputListView::RequestRefresh()
{
	if (TSharedPtr<STimedDataMonitorPanel> OwnerPanelPin = OwnerPanel.Pin())
	{
		OwnerPanelPin->RequestRefresh();
	}
}


void STimedDataInputListView::MarkSettingsDirty()
{
	if (TSharedPtr<STimedDataMonitorPanel> OwnerPanelPin = OwnerPanel.Pin())
	{
		OwnerPanelPin->MarkSettingsDirty();
	}
}


void STimedDataInputListView::UpdateCachedValue()
{
	if (bRebuildListRequested)
	{
		RebuildSources();
		RebuildList();
		bRebuildListRequested = false;
	}

	for (FTimedDataInputTableRowDataPtr& RowDataPtr : ListItemsSource)
	{
		RowDataPtr->UpdateCachedValue();
	}

	for (int32 Index = ListRowWidgets.Num() - 1; Index >= 0; --Index)
	{
		const TSharedPtr<STimedDataInputTableRow> Row = ListRowWidgets[Index].Pin();
		if (Row)
		{
			Row->UpdateCachedValue();
		}
		else
		{
			ListRowWidgets.RemoveAtSwap(Index);
		}
	}
}


void STimedDataInputListView::RequestRebuildSources()
{
	bRebuildListRequested = true;
}


void STimedDataInputListView::RebuildSources()
{
	ListItemsSource.Reset();

	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	TArray<FTimedDataMonitorInputIdentifier> Inputs = TimedDataMonitorSubsystem->GetAllInputs();
	for (const FTimedDataMonitorInputIdentifier& InputIdentifier : Inputs)
	{
		TSharedRef<FTimedDataInputTableRowData> ParentRowData = MakeShared<FTimedDataInputTableRowData>(InputIdentifier);
		ListItemsSource.Add(ParentRowData);

		TArray<FTimedDataMonitorChannelIdentifier> Channels = TimedDataMonitorSubsystem->GetInputChannels(InputIdentifier);
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Channels)
		{
			TSharedRef<FTimedDataInputTableRowData> ChildRowData = MakeShared<FTimedDataInputTableRowData>(ChannelIdentifier);
			ParentRowData->InputChildren.Add(ChildRowData);
		}
	}

	struct FSortNamesAlphabetically
	{
		bool operator()(const TSharedPtr<FTimedDataInputTableRowData>& LHS, const TSharedPtr<FTimedDataInputTableRowData>& RHS) const
		{
			return (LHS->DisplayName.CompareTo(RHS->DisplayName) < 0);
		}
	};

	ListItemsSource.Sort(FSortNamesAlphabetically());

	for (FTimedDataInputTableRowDataPtr& TableRowData : ListItemsSource)
	{
		TableRowData->UpdateCachedValue();
	}

	RequestTreeRefresh();
}


FTimedDataMonitorInputIdentifier STimedDataInputListView::GetSelectedInputIdentifier() const
{
	TArray<FTimedDataInputTableRowDataPtr> SelectedRows = GetSelectedItems();
	if (SelectedRows.Num() > 0)
	{
		return SelectedRows[0]->InputIdentifier;
	}
	return FTimedDataMonitorInputIdentifier();
}


ECheckBoxState STimedDataInputListView::GetAllEnabledCheckState() const
{
	return ECheckBoxState::Checked;
}


void STimedDataInputListView::OnToggleAllEnabledCheckState(ECheckBoxState CheckBoxState)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	bool bIsEnabled = CheckBoxState == ECheckBoxState::Checked;
	for (const FTimedDataInputTableRowDataPtr& RowDataPtr : ListItemsSource)
	{
		TimedDataMonitorSubsystem->SetInputEnabled(RowDataPtr->InputIdentifier, bIsEnabled);
	}

	if (TSharedPtr<STimedDataMonitorPanel> OwnerPanelPin = OwnerPanel.Pin())
	{
		OwnerPanelPin->MarkSettingsDirty();
	}
}


TSharedRef<ITableRow> STimedDataInputListView::OnGenerateRow(FTimedDataInputTableRowDataPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<STimedDataInputTableRow> Row = SNew(STimedDataInputTableRow, OwnerTable, SharedThis<STimedDataInputListView>(this))
		.Item(InItem);
	ListRowWidgets.Add(Row);
	return Row;
}


void STimedDataInputListView::ReleaseListViewWidget(const TSharedRef<ITableRow>& Row)
{
	TSharedRef<STimedDataInputTableRow> RefRow = StaticCastSharedRef<STimedDataInputTableRow>(Row);
	TWeakPtr<STimedDataInputTableRow> WeakRow = RefRow;
	ListRowWidgets.RemoveSingleSwap(WeakRow);
}



void STimedDataInputListView::GetChildrenForInfo(FTimedDataInputTableRowDataPtr InItem, TArray<FTimedDataInputTableRowDataPtr>& OutChildren)
{
	OutChildren = InItem->InputChildren;
}


void STimedDataInputListView::OnSelectionChanged(FTimedDataInputTableRowDataPtr InItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		if (InItem && !InItem->bIsInput)
		{
			ClearSelection();
		}
	}
}


bool STimedDataInputListView::OnIsSelectableOrNavigable(FTimedDataInputTableRowDataPtr InItem) const
{
	return InItem && !InItem->bIsInput;
}


#undef LOCTEXT_NAMESPACE
