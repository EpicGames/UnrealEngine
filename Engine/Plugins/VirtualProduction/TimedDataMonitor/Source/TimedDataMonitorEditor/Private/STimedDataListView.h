// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "Widgets/Views/STreeView.h"

#include "STimedDataListView.generated.h"

struct FTimedDataMonitorInputIdentifier;

enum class ETimedDataInputEvaluationType : uint8;
struct FTimedDataInputTableRowData;
struct FSlateBrush;
class STimedDataInputListView;
class STimedDataInputTableRow;
class STimedDataMonitorPanel;
class STimingDiagramWidget;

UENUM()
enum class ETimedDataInputEvaluationOffsetType : uint8
{
	/** The input offset is specified in seconds. */
	Seconds UMETA(DisplayName = "s"),
	/** The input offset is specified in frames. */
	Frames UMETA(DisplayName = "f"),
};


/**
 * 
 */
using FTimedDataInputTableRowDataPtr = TSharedPtr<FTimedDataInputTableRowData>;


/**
 *
 */
class STimedDataInputTableRow : public SMultiColumnTableRow<FTimedDataInputTableRowDataPtr>
{
	using Super = SMultiColumnTableRow<FTimedDataInputTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(STimedDataInputTableRow) { }
		SLATE_ARGUMENT(FTimedDataInputTableRowDataPtr, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwerTableView, const TSharedRef<STimedDataInputListView>& OwnerTreeView);

public:
	void UpdateCachedValue();

private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	ECheckBoxState GetEnabledCheckState() const;
	void OnEnabledCheckStateChanged(ECheckBoxState NewState);

	FText GetStateGlyphs() const;
	FSlateColor GetStateColorAndOpacity() const;

	FText GetDescription() const;
	double GetFloatEvaluationOffset() const;
	int32 GetIntEvaluationOffset() const;
	FText GetEvaluationOffsetText() const;
	void SetEvaluationOffset(double NewValue, ETextCommit::Type CommitType);
	void SetEvaluationOffset(int32 NewValue, ETextCommit::Type CommitType);
	int32 GetBufferSize() const;
	FText GetBufferSizeText() const;
	void SetBufferSize(int32 NewValue, ETextCommit::Type CommitType);
	bool CanEditBufferSize() const;
	int32 GetCurrentSampleCount() const;
	TSharedRef<SWidget> OnEvaluationImageBuildMenu();
	const FSlateBrush* GetEvaluationImage() const;
	void SetInputEvaluationType(ETimedDataInputEvaluationType EvaluationType);

	/** Queries about buffer stats to display */
	FText GetBufferUnderflowCount() const;
	FText GetBufferOverflowCount() const;
	FText GetFrameDroppedCount() const;

	void OnEvaluationOffsetTypeChanged(int32 NewValue, ESelectInfo::Type);

private:
	FTimedDataInputTableRowDataPtr Item;
	TSharedPtr<STimedDataInputListView> OwnerTreeView;
	TSharedPtr<STimingDiagramWidget> DiagramWidget;
};


/**
 *
 */
class STimedDataInputListView : public STreeView<FTimedDataInputTableRowDataPtr>
{
	using Super = STreeView<FTimedDataInputTableRowDataPtr>;

public:
	SLATE_BEGIN_ARGS(STimedDataInputListView) {}
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> OwnerPanel);
	virtual ~STimedDataInputListView();

	void RequestRefresh();
	void UpdateCachedValue();
	void MarkSettingsDirty();

	FTimedDataMonitorInputIdentifier GetSelectedInputIdentifier() const;

private:
	void RequestRebuildSources();
	void RebuildSources();

	ECheckBoxState GetAllEnabledCheckState() const;
	void OnToggleAllEnabledCheckState(ECheckBoxState CheckBoxState);
	TSharedRef<ITableRow> OnGenerateRow(FTimedDataInputTableRowDataPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	void ReleaseListViewWidget(const TSharedRef<ITableRow>& Row);
	void GetChildrenForInfo(FTimedDataInputTableRowDataPtr InItem, TArray<FTimedDataInputTableRowDataPtr>& OutChildren);
	void OnSelectionChanged(FTimedDataInputTableRowDataPtr InItem, ESelectInfo::Type SelectInfo);
	bool OnIsSelectableOrNavigable(FTimedDataInputTableRowDataPtr InItem) const;


private:
	TWeakPtr<STimedDataMonitorPanel> OwnerPanel;

	TArray<FTimedDataInputTableRowDataPtr> ListItemsSource;
	TArray<TWeakPtr<STimedDataInputTableRow>> ListRowWidgets;
	bool bRebuildListRequested = true;
};
