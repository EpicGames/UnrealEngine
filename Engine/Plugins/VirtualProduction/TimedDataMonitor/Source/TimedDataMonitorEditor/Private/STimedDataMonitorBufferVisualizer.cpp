// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimedDataMonitorBufferVisualizer.h"

#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Misc/App.h"
#include "STimingDiagramWidget.h"
#include "TimedDataMonitorEditorSettings.h"
#include "TimedDataMonitorSubsystem.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "TimedDataBufferVisualizer"

class STimedDataMonitorBufferVisualizerListView;

/* TimedDataBufferVisualizer
 *****************************************************************************/

 namespace TimedDataBufferVisualizer
 {
	static const FName HeaderIdName_DisplayName = "DisplayName";
	static const FName HeaderIdName_Visual = "Visual";
 }


/* FTimedDataMonitorBuffervisualizerRow
 *****************************************************************************/

struct FTimedDataMonitorBuffervisualizerItem
{
	FTimedDataMonitorChannelIdentifier ChannelIdentifier;
	FTimedDataMonitorInputIdentifier InputIdentifier;
	FText ChannelDisplayName;
};


/* STimedDataMonitorBufferVisualizerRow
 *****************************************************************************/
class STimedDataMonitorBufferVisualizerRow : public SMultiColumnTableRow<TSharedPtr<FTimedDataMonitorBuffervisualizerItem>>
{
public:
	SLATE_BEGIN_ARGS(STimedDataMonitorBufferVisualizerRow) {}
		SLATE_ARGUMENT(TSharedPtr<FTimedDataMonitorBuffervisualizerItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<STimedDataMonitorBufferVisualizerListView>& InListView)
	{
		Item = Args._Item;
		ListView = InListView;

		SMultiColumnTableRow<TSharedPtr<FTimedDataMonitorBuffervisualizerItem>>::Construct(
			FSuperRowType::FArguments()
			.Padding(0.f),
			OwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == TimedDataBufferVisualizer::HeaderIdName_DisplayName)
		{
			return SNew(SBox)
				.MinDesiredHeight(40)
				[
					SNew(STextBlock)
					.Text(Item->ChannelDisplayName)
				];
		}
		else if (ColumnName == TimedDataBufferVisualizer::HeaderIdName_Visual)
		{
			return SAssignNew(DiagramWidget, STimingDiagramWidget, false)
				.ChannelIdentifier(Item->ChannelIdentifier)
				.ShowFurther(true)
				.ShowMean(true)
				.ShowSigma(true)
				.ShowSnapshot(true)
				.UseNiceBrush(false)
				.SizePerSeconds(this, &STimedDataMonitorBufferVisualizerRow::GetSizeOfSeconds);
		}

		return SNullWidget::NullWidget;
	}

	void UpdateCachedValue()
	{
		if (DiagramWidget)
		{
			DiagramWidget->UpdateCachedValue();
		}
	}

	float GetSizeOfSeconds() const;

private:
	TSharedPtr<FTimedDataMonitorBuffervisualizerItem> Item;
	TSharedPtr<STimingDiagramWidget> DiagramWidget;
	TWeakPtr<STimedDataMonitorBufferVisualizerListView> ListView;
};


/* STimedDataMonitorBufferVisualizerListView
 *****************************************************************************/

class STimedDataMonitorBufferVisualizerListView : public SListView<TSharedPtr<FTimedDataMonitorBuffervisualizerItem>>
{
public:
	virtual bool SupportsKeyboardFocus() const override
	{
		return false;
	}
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override
	{
		return FReply::Unhandled();
	}
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		float Scale = 2.f;
		float MinValue = 2.f;

		if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
		{
			Scale *= ControlKeyModifier;
		}

		SizePerSeconds = FMath::Max(MinValue, SizePerSeconds + MouseEvent.GetWheelDelta() * Scale);
		return FReply::Handled();
	}

	float GetSizeOfSeconds() const
	{
		return SizePerSeconds;
	}

	float SizePerSeconds = 100.f;
	float ControlKeyModifier = 50.f;
};


float STimedDataMonitorBufferVisualizerRow::GetSizeOfSeconds() const
{
	if (TSharedPtr<STimedDataMonitorBufferVisualizerListView> ListViewPin = ListView.Pin())
	{
		return ListViewPin->GetSizeOfSeconds();
	}
	return 100.f;
}

/* STimedDataMonitorBufferVisualizer implementation
 *****************************************************************************/

void STimedDataMonitorBufferVisualizer::Construct(const FArguments& InArgs)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);
	TimedDataMonitorSubsystem->OnIdentifierListChanged().AddSP(this, &STimedDataMonitorBufferVisualizer::RequestRefresh);

	ListViewWidget =
		SNew(STimedDataMonitorBufferVisualizerListView)
		.SelectionMode(ESelectionMode::None)
		.ListItemsSource(&ListItemsSource)
		.OnGenerateRow(this, &STimedDataMonitorBufferVisualizer::MakeListViewWidget)
		.OnRowReleased(this, &STimedDataMonitorBufferVisualizer::ReleaseListViewWidget)
		.HeaderRow(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(TimedDataBufferVisualizer::HeaderIdName_DisplayName)
			.FillWidth(0.2f)
			.DefaultLabel(FText())
			+SHeaderRow::Column(TimedDataBufferVisualizer::HeaderIdName_Visual)
			.FillWidth(0.8f)
			.DefaultLabel(FText())
		);

	ChildSlot
	[
		SNew(SScrollBorder, ListViewWidget.ToSharedRef())
		[
			ListViewWidget.ToSharedRef()
		]
	];
}


STimedDataMonitorBufferVisualizer::~STimedDataMonitorBufferVisualizer()
{
	if (UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>())
	{
		TimedDataMonitorSubsystem->OnIdentifierListChanged().RemoveAll(this);
	}
}


void STimedDataMonitorBufferVisualizer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	double RefreshTimer = GetDefault<UTimedDataMonitorEditorSettings>()->RefreshRate;
	if (bRefreshRequested || (FApp::GetCurrentTime() - LastRefreshPlatformSeconds > RefreshTimer))
	{
		LastRefreshPlatformSeconds = FApp::GetCurrentTime();
		bRefreshRequested = false;

		RebuiltListItemsSource();
		for (int32 Index = ListRowWidgets.Num() - 1; Index >= 0; --Index)
		{
			TSharedPtr<STimedDataMonitorBufferVisualizerRow> Row = ListRowWidgets[Index].Pin();
			if (Row == nullptr)
			{
				ListRowWidgets.RemoveAtSwap(Index);
			}
			else
			{
				Row->UpdateCachedValue();
			}
		}
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


void STimedDataMonitorBufferVisualizer::RequestRefresh()
{
	bRefreshRequested = true;
}


void STimedDataMonitorBufferVisualizer::RebuiltListItemsSource()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	TArray<FTimedDataMonitorChannelIdentifier> AllChannelIdentifiers = TimedDataMonitorSubsystem->GetAllEnabledChannels();

	bool bListModified = false;

	// Remove from ListItemsSource all item that are not in the AllChannelIdentifiers list
	for (int32 Index = ListItemsSource.Num() - 1; Index >= 0; --Index)
	{
		int32 NumberRemoved = AllChannelIdentifiers.RemoveSingleSwap(ListItemsSource[Index]->ChannelIdentifier, EAllowShrinking::No);
		if (NumberRemoved == 0)
		{
			bListModified = true;
			ListItemsSource.RemoveAtSwap(Index);
		}
	}

	// Add missing channel
	for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : AllChannelIdentifiers)
	{
		TSharedPtr<FTimedDataMonitorBuffervisualizerItem> ItemPtr = MakeShared<FTimedDataMonitorBuffervisualizerItem>();
		ItemPtr->ChannelIdentifier = ChannelIdentifier;
		ItemPtr->InputIdentifier = TimedDataMonitorSubsystem->GetChannelInput(ChannelIdentifier);
		ItemPtr->ChannelDisplayName = TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier);
		ListItemsSource.Add(ItemPtr);
		bListModified = true;
	}

	if (bListModified)
	{
		struct FSortNamesAlphabetically
		{
			bool operator()(const TSharedPtr<FTimedDataMonitorBuffervisualizerItem>& LHS, const TSharedPtr<FTimedDataMonitorBuffervisualizerItem>& RHS) const
			{
				return (LHS->ChannelDisplayName.CompareTo(RHS->ChannelDisplayName) < 0);
			}
		};

		ListItemsSource.Sort(FSortNamesAlphabetically());
		ListViewWidget->RequestListRefresh();
	}
}


TSharedRef<ITableRow> STimedDataMonitorBufferVisualizer::MakeListViewWidget(TSharedPtr<FTimedDataMonitorBuffervisualizerItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<STimedDataMonitorBufferVisualizerRow> Row = SNew(STimedDataMonitorBufferVisualizerRow, OwnerTable, StaticCastSharedRef<STimedDataMonitorBufferVisualizerListView>(OwnerTable))
		.Item(Item);
	ListRowWidgets.Add(Row);
	return Row;
}


void STimedDataMonitorBufferVisualizer::ReleaseListViewWidget(const TSharedRef<ITableRow>& Row)
{
	TSharedRef<STimedDataMonitorBufferVisualizerRow> RefRow = StaticCastSharedRef<STimedDataMonitorBufferVisualizerRow>(Row);
	TWeakPtr<STimedDataMonitorBufferVisualizerRow> WeakRow = RefRow;
	ListRowWidgets.RemoveSingleSwap(WeakRow);
}

#undef LOCTEXT_NAMESPACE
