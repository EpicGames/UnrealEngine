// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FTimedDataMonitorBuffervisualizerItem;
class ITableRow;
class STableViewBase;
class STimedDataMonitorBufferVisualizerListView;
class STimedDataMonitorBufferVisualizerRow;

/**
 * Implements the contents of the viewer tab in the TimecodeSynchronizer editor.
 */
class STimedDataMonitorBufferVisualizer : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;
public:

	SLATE_BEGIN_ARGS(STimedDataMonitorBufferVisualizer) { }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual ~STimedDataMonitorBufferVisualizer();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void RequestRefresh();
	void RebuiltListItemsSource();
	TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<FTimedDataMonitorBuffervisualizerItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void ReleaseListViewWidget(const TSharedRef<ITableRow>&);

private:
	TSharedPtr<STimedDataMonitorBufferVisualizerListView> ListViewWidget;
	TArray<TWeakPtr<STimedDataMonitorBufferVisualizerRow>> ListRowWidgets;
	TArray<TSharedPtr<FTimedDataMonitorBuffervisualizerItem>> ListItemsSource;
	

	double LastRefreshPlatformSeconds = 0.0;
	bool bRefreshRequested = true;
};
