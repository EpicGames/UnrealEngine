// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimedDataMonitorPanel.h"

#include "Editor/EditorPerformanceSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "IMessageLogListing.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "MessageLogModule.h"
#include "Misc/App.h"
#include "TimedDataMonitorSubsystem.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "EditorFontGlyphs.h"
#include "Features/IModularFeatures.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ILiveLinkClient.h"
#include "LiveLinkClient.h"
#include "LiveLinkPreset.h"
#include "LiveLinkSettings.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Engine/TimecodeProvider.h"
#include "ScopedTransaction.h"
#include "TimedDataMonitorEditorStyle.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "SlateOptMacros.h"
#include "STimedDataGenlock.h"
#include "STimedDataListView.h"
#include "STimedDataMonitorBufferVisualizer.h"
#include "STimedDataTimecodeProvider.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "STimedDataMonitorPanel"


TWeakPtr<STimedDataMonitorPanel> STimedDataMonitorPanel::WidgetInstance;

namespace Utilities
{
	static const FName NAME_App = FName("TimedDataSourceMonitorPanelApp");
	static const FName NAME_TimedDataMonitorBuffers = FName("TimedDataMonitorBuffers");
	static const FName NAME_LogName = "Timed Data Monitor";

	TSharedRef<SDockTab> CreateApp(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(STimedDataMonitorPanel)
			];
	}

	TSharedRef<SDockTab> CreateBuffersVisualizer(const FSpawnTabArgs&)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(STimedDataMonitorBufferVisualizer)
			];
	}
}


const FName STimedDataMonitorPanel::TabName = "TimedDataSourceMonitorPanelApp";


void STimedDataMonitorPanel::RegisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager, TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	if (TabManager)
	{
		TabManager->RegisterTabSpawner(Utilities::NAME_App, FOnSpawnTab::CreateStatic(&Utilities::CreateApp))
			.SetDisplayName(LOCTEXT("TabTitle", "Timed Data Monitor"))
			.SetTooltipText(LOCTEXT("TooltipText", "Monitor inputs that can be time synchronized"))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), "Img.TimedDataMonitor.Icon"));
	}

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(Utilities::NAME_TimedDataMonitorBuffers, FOnSpawnTab::CreateStatic(&Utilities::CreateBuffersVisualizer))
		.SetDisplayName(LOCTEXT("BufferVisualizerTitle", "Buffer Visualizer"))
		.SetTooltipText(LOCTEXT("BufferVisualizerTooltip", "Open buffer visualizer tab for Timed Data Monitor."))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}

void STimedDataMonitorPanel::UnregisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager)
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(Utilities::NAME_TimedDataMonitorBuffers);

	if (TabManager)
	{
		TabManager->UnregisterTabSpawner(Utilities::NAME_App);
	}
}


TSharedPtr<STimedDataMonitorPanel> STimedDataMonitorPanel::GetPanelInstance()
{
	return STimedDataMonitorPanel::WidgetInstance.Pin();
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimedDataMonitorPanel::Construct(const FArguments& InArgs)
{
	WidgetInstance = StaticCastSharedRef<STimedDataMonitorPanel>(AsShared());

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (!MessageLogModule.IsRegisteredLogListing(Utilities::NAME_LogName))
	{
		MessageLogListing = MessageLogModule.CreateLogListing(Utilities::NAME_LogName);
	}
	else
	{
		MessageLogListing = MessageLogModule.GetLogListing(Utilities::NAME_LogName);
	}

	BuildCalibrationArray();

	TSharedRef<SWidget> MessageLogListingWidget = MessageLogListing.IsValid() ? MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef()) : SNullWidget::NullWidget;

	TSharedPtr<SWidget> PerformanceWidget;
	{
		FProperty* PerformanceThrottlingProperty = FindFieldChecked<FProperty>(UEditorPerformanceSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bThrottleCPUWhenNotForeground));
		PerformanceThrottlingProperty->GetDisplayNameText();
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("PropertyName"), PerformanceThrottlingProperty->GetDisplayNameText());
		FText PerformanceWarningText = FText::Format(LOCTEXT("PerformanceWarningMessage", "Warning: The editor setting '{PropertyName}' is currently enabled\nThis will stop editor windows from updating in realtime while the editor is not in focus"), Arguments);

		PerformanceWidget = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
			.BorderBackgroundColor(FColor(166, 137, 0))
			.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
			.Visibility(this, &STimedDataMonitorPanel::ShowEditorPerformanceThrottlingWarning)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(PerformanceWarningText)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.ShadowColorAndOpacity(FLinearColor::Black.CopyWithNewOpacity(0.3f))
					.ShadowOffset(FVector2D::UnitVector)
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.f, 0.f, 10.f, 0.f))
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.OnClicked(this, &STimedDataMonitorPanel::DisableEditorPerformanceThrottling)
					.Text(LOCTEXT("PerformanceWarningDisable", "Disable"))
				]
			];
	}

	TSharedRef<SWidget> Toolbar = 
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(7.f, 5.f))
		[
			SNew(SHorizontalBox)
			// Calibrate button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SButton)
					.ToolTipText(this, &STimedDataMonitorPanel::GetCalibrateButtonTooltip)
					.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
					.ContentPadding(FMargin(4, 0))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked(this, &STimedDataMonitorPanel::OnCalibrateClicked)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(this, &STimedDataMonitorPanel::GetCalibrateButtonImage)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(10.f, 4.f, 4.f, 4.f)
						[
							SNew(STextBlock)
							.TextStyle(FTimedDataMonitorEditorStyle::Get(), "TextBlock.Large")
							.Text(this, &STimedDataMonitorPanel::GetCalibrateButtonText)
							.ColorAndOpacity(FLinearColor::White)
						]
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnGetMenuContent(this, &STimedDataMonitorPanel::OnCalibrateBuildMenu)
				]
			]
			// reset errors button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SButton)
					.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
					.ToolTipText(LOCTEXT("ResetErrors_ToolTip", "Reset the buffer statistics, message or evaluation time (base on settings)."))
					.ContentPadding(FMargin(4, 4))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked(this, &STimedDataMonitorPanel::OnResetErrorsClicked)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
						.Text(FEditorFontGlyphs::Eraser)
						.ColorAndOpacity(FLinearColor::White)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnGetMenuContent(this, &STimedDataMonitorPanel::OnResetBuildMenu)
				]
			]
			// Save preset button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(SButton)
				.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("SavePreset_ToolTip", "Save all monitored input settings and the Live Link preset"))
				.ContentPadding(FMargin(4, 4))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &STimedDataMonitorPanel::OnSavePreset)
				[
					SNew(SImage)
					.Image(this, &STimedDataMonitorPanel::GetSaveButtonBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			// show buffers
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(SButton)
				.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("ShowBuffers_ToolTip", "Open the buffer visualizer"))
				.ContentPadding(FMargin(4, 2))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &STimedDataMonitorPanel::OnShowBuffersClicked)
				[
					SNew(SImage)
					.Image(FTimedDataMonitorEditorStyle::Get().GetBrush("Img.BufferVisualization"))
				]
			]
			// Settings button
			+ SHorizontalBox::Slot()
			.Padding(4.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("ShowUserSettings_Tip", "Show the general user settings"))
				.ContentPadding(FMargin(4, 4))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &STimedDataMonitorPanel::OnGeneralUserSettingsClicked)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text(FEditorFontGlyphs::Cogs)
					.ColorAndOpacity(FLinearColor::White)
				]
			]
			// Spacer
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
			// Status
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.18"))
				.Text(FEditorFontGlyphs::Circle)
				.ColorAndOpacity(this, &STimedDataMonitorPanel::GetEvaluationStateColorAndOpacity)
			]
			// Status test
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 4.f, 6.f, 4.f)
			[
				SNew(STextBlock)
				.MinDesiredWidth(100)
				.TextStyle(FTimedDataMonitorEditorStyle::Get(), "TextBlock.Large")
				.Text(this, &STimedDataMonitorPanel::GetEvaluationStateText)
				.ColorAndOpacity(FLinearColor::White)
			]
		];

	FSlateFontInfo SectionHeaderFont = FCoreStyle::Get().GetFontStyle(TEXT("EmbossedText"));
	SectionHeaderFont.Size += 4;

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BorderBackgroundColor(FLinearColor::Gray) // Darken the outer border
			.Padding(0.f)
			[
				SNew(SVerticalBox)
				// Toolbar
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					Toolbar
				]
				// Timing element
				+ SVerticalBox::Slot()
				.Padding(4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.5f)
					.Padding(FMargin(0.f, 0.f, 2.5f, 0.f))
					[
						SNew(SBorder)
						.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
						.Padding(FMargin(5.f, 2.f))
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Font(SectionHeaderFont)
								.Text(LOCTEXT("GenlockLabel", "Genlock"))
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SAssignNew(TimedDataGenlockWidget, STimedDataGenlock, SharedThis<STimedDataMonitorPanel>(this))
							]
						]
					]
					+ SHorizontalBox::Slot()
					.FillWidth(0.5f)
					.Padding(FMargin(2.5f, 0.f, 0.f, 0.f))
					[
						SNew(SBorder)
						.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
						.Padding(FMargin(5.f, 2.f))
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Font(SectionHeaderFont)
								.Text(LOCTEXT("TimecodeLabel", "Timecode Provider"))
							]
							+ SVerticalBox::Slot()
							[
								SAssignNew(TimedDataTimecodeWidget, STimedDataTimecodeProvider, SharedThis<STimedDataMonitorPanel>(this))
							]
						]
					]
				]
				// Sources list
				+ SVerticalBox::Slot()
				.Padding(5.f, 2.5f)
				.FillHeight(1.f)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Vertical)
					+ SScrollBox::Slot()
					[
						SAssignNew(TimedDataSourceList, STimedDataInputListView, SharedThis<STimedDataMonitorPanel>(this))
						.OnContextMenuOpening(this, &STimedDataMonitorPanel::OnDataListConstructContextMenu)
					]
				]
				// Message log
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
					.Visibility(this, &STimedDataMonitorPanel::ShowMessageLog)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							MessageLogListingWidget
						]
					]
				]
				// Show performance warning
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.f, 0.f, 10.f, 10.f)
				[
					PerformanceWidget.ToSharedRef()
				]
			]
		]
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
			.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f))
			.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
			.Visibility(this, &STimedDataMonitorPanel::GetThrobberVisibility)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SCircularThrobber)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SButton)
						.ContentPadding(FMargin(4, 2))
						.OnClicked(this, &STimedDataMonitorPanel::OnCancelCalibration)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CancelLabel", "Cancel"))
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void STimedDataMonitorPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	double RefreshTimer = GetDefault<UTimedDataMonitorEditorSettings>()->RefreshRate;
	if (bRefreshRequested || (FApp::GetCurrentTime() - LastCachedValueUpdateTime > RefreshTimer))
	{
		bRefreshRequested = false;
		LastCachedValueUpdateTime = FApp::GetCurrentTime();

		{
			UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
			check(TimedDataMonitorSubsystem);
			CachedGlobalEvaluationState = TimedDataMonitorSubsystem->GetEvaluationState();
		}

		if (TimedDataGenlockWidget)
		{
			TimedDataGenlockWidget->UpdateCachedValue(InDeltaTime);
		}
		if (TimedDataTimecodeWidget)
		{
			TimedDataTimecodeWidget->UpdateCachedValue();
		}
		if (TimedDataSourceList)
		{
			TimedDataSourceList->UpdateCachedValue();
		}
	}

	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


void STimedDataMonitorPanel::BuildCalibrationArray()
{
	CalibrationUIAction[(int32)ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode] = FUIAction(
		FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::CalibrateWithTimecode));
	CalibrationUIAction[(int32)ETimedDataMonitorEditorCalibrationType::TimeCorrection] = FUIAction(
		FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::ApplyTimeCorrectionAll));
	CalibrationSlateIcon[(int32)ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode] = FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), "Img.Calibration");
	CalibrationSlateIcon[(int32)ETimedDataMonitorEditorCalibrationType::TimeCorrection] = FSlateIcon(FTimedDataMonitorEditorStyle::Get().GetStyleSetName(), "Img.TimeCorrection");
	CalibrationName[(int32)ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode] = LOCTEXT("CalibrateButtonLabel", "Calibrate");
	CalibrationName[(int32)ETimedDataMonitorEditorCalibrationType::TimeCorrection] = LOCTEXT("JamWithTimeButtonLabel", "Time Correction");
	CalibrationTooltip[(int32)ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode] = LOCTEXT("CalibrateButtonTooltip", "Find range of frames that all inputs has. If it can be found, change the Timecode Provider delay, resize buffers or delay them to accomadate.");
	CalibrationTooltip[(int32)ETimedDataMonitorEditorCalibrationType::TimeCorrection] = LOCTEXT("JamWithTimeButtonTooltip", "Set the offset of all inputs to match the Timecode value of the Timecode Provider or the Platform Time, depending on their evaluation type. It may resize buffers if they are too small.");
}


FReply STimedDataMonitorPanel::OnCalibrateClicked()
{
	ETimedDataMonitorEditorCalibrationType CalibrationType = GetDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType;
	if (CalibrationUIAction[(int32)CalibrationType].CanExecuteAction.Execute())
	{
		CalibrationUIAction[(int32)CalibrationType].ExecuteAction.Execute();
	}

	return FReply::Handled();
}


TSharedRef<SWidget> STimedDataMonitorPanel::OnCalibrateBuildMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 Index = 0; Index < CalibrationArrayCount; ++Index)
	{
		MenuBuilder.AddMenuEntry(CalibrationName[Index], CalibrationTooltip[Index], CalibrationSlateIcon[Index], CalibrationUIAction[Index], NAME_None, EUserInterfaceActionType::RadioButton);
	}

	return MenuBuilder.MakeWidget();
}


FText STimedDataMonitorPanel::GetCalibrateButtonTooltip() const
{
	ETimedDataMonitorEditorCalibrationType CalibrationType = GetDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType;
	return CalibrationTooltip[(int32)CalibrationType];
}


const FSlateBrush* STimedDataMonitorPanel::GetCalibrateButtonImage() const
{
	ETimedDataMonitorEditorCalibrationType CalibrationType = GetDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType;
	return CalibrationSlateIcon[(int32)CalibrationType].GetIcon();
}


FText STimedDataMonitorPanel::GetCalibrateButtonText() const
{
	ETimedDataMonitorEditorCalibrationType CalibrationType = GetDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType;
	return CalibrationName[(int32)CalibrationType];
}


FReply STimedDataMonitorPanel::OnResetErrorsClicked()
{
	if (IsResetBufferStatChecked())
	{
		OnResetBufferStatClicked();
	}
	if (IsClearMessageChecked())
	{
		OnClearMessageClicked();
	}
	if (IsResetEvaluationChecked())
	{
		OnResetAllEvaluationTimeClicked();
	}

	return FReply::Handled();
}


TSharedRef<SWidget> STimedDataMonitorPanel::OnResetBuildMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Reset the buffer statistics, message and evaluation time
	MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetBufferStatisticsLabel", "Reset buffer statistics"),
			LOCTEXT("ResetBufferStatisticsTooltip", "Reset the buffer statistics."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::OnResetBufferStatClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimedDataMonitorPanel::IsResetBufferStatChecked))
			);
	MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetMessageLabel", "Clear messages"),
			LOCTEXT("ResetMessageTooltip", "Clear the message list."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::OnClearMessageClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimedDataMonitorPanel::IsClearMessageChecked))
			);

	MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetEvaluationTimeLabel", "Reset Evaluation Time"),
			LOCTEXT("ResetEvaluationTimeTooltip", "Reset all the evaluation time for all input."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::OnResetAllEvaluationTimeClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &STimedDataMonitorPanel::IsResetEvaluationChecked))
			);

	return MenuBuilder.MakeWidget();
}


void STimedDataMonitorPanel::OnResetBufferStatClicked()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);
	TimedDataMonitorSubsystem->ResetAllBufferStats();
}


bool STimedDataMonitorPanel::IsResetBufferStatChecked() const
{
	return GetDefault<UTimedDataMonitorEditorSettings>()->bResetBufferStatEnabled;
}


void STimedDataMonitorPanel::OnClearMessageClicked()
{
	MessageLogListing->ClearMessages();
}


bool STimedDataMonitorPanel::IsClearMessageChecked() const
{
	return GetDefault<UTimedDataMonitorEditorSettings>()->bClearMessageEnabled;
}


void STimedDataMonitorPanel::OnResetAllEvaluationTimeClicked()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);
	TArray<FTimedDataMonitorInputIdentifier> AllInputs = TimedDataMonitorSubsystem->GetAllInputs();
	for (const FTimedDataMonitorInputIdentifier& Input : AllInputs)
	{
		TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(Input, 0.f);
	}
	MarkSettingsDirty();
}


bool STimedDataMonitorPanel::IsResetEvaluationChecked() const
{
	return GetDefault<UTimedDataMonitorEditorSettings>()->bResetEvaluationTimeEnabled;
}


FReply STimedDataMonitorPanel::OnShowBuffersClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(Utilities::NAME_TimedDataMonitorBuffers));
	return FReply::Handled();
}


FReply STimedDataMonitorPanel::OnGeneralUserSettingsClicked()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "Plugins", "Timed Data Monitor");
	return FReply::Handled();
}


FSlateColor STimedDataMonitorPanel::GetEvaluationStateColorAndOpacity() const
{
	return (CachedGlobalEvaluationState == ETimedDataMonitorEvaluationState::InsideRange) ? FLinearColor::Green : FLinearColor::Red;
}


FText STimedDataMonitorPanel::GetEvaluationStateText() const
{
	switch(CachedGlobalEvaluationState)
	{
	case ETimedDataMonitorEvaluationState::NoSample:
		return LOCTEXT("EvalState_NoSample", "No Samples");
	case ETimedDataMonitorEvaluationState::OutsideRange:
		return LOCTEXT("EvalState_OutsideRange", "Outside Range");
	case ETimedDataMonitorEvaluationState::InsideRange:
		return LOCTEXT("EvalState_InsideRange", "Synchronized");
	}

	return LOCTEXT("EvalState_Disabled", "Disabled");
}


TSharedPtr<SWidget> STimedDataMonitorPanel::OnDataListConstructContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("TimeCorrection", "Time Correction"),
		FText(),
		CalibrationSlateIcon[(int32)ETimedDataMonitorEditorCalibrationType::TimeCorrection],
		FUIAction(
			FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::ApplyTimeCorrectionOnSelection),
			FCanExecuteAction::CreateSP(this, &STimedDataMonitorPanel::IsSourceListSectionValid)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResetEvaluationTime", "Reset Time Correction"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &STimedDataMonitorPanel::ResetTimeCorrectionOnSelection),
			FCanExecuteAction::CreateSP(this, &STimedDataMonitorPanel::IsSourceListSectionValid)
		)
	);

	return MenuBuilder.MakeWidget();
}


bool STimedDataMonitorPanel::IsSourceListSectionValid() const
{
	return TimedDataSourceList && TimedDataSourceList->GetSelectedInputIdentifier().IsValid();
}


void STimedDataMonitorPanel::ApplyTimeCorrectionOnSelection()
{
	if (TimedDataSourceList)
	{
		const FTimedDataMonitorInputIdentifier SelectedInput = TimedDataSourceList->GetSelectedInputIdentifier();
		if (SelectedInput.IsValid())
		{
			ApplyTimeCorrection(SelectedInput);
			MarkSettingsDirty();
		}
	}
}


void STimedDataMonitorPanel::ResetTimeCorrectionOnSelection()
{
	if (TimedDataSourceList)
	{
		const FTimedDataMonitorInputIdentifier SelectedInput = TimedDataSourceList->GetSelectedInputIdentifier();
		if (SelectedInput.IsValid())
		{
			UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
			TimedDataMonitorSubsystem->SetInputEvaluationOffsetInSeconds(SelectedInput, 0.f);
			MarkSettingsDirty();
		}
	}
}


EVisibility STimedDataMonitorPanel::ShowMessageLog() const
{
	return MessageLogListing && MessageLogListing->NumMessages(EMessageSeverity::Info) > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility STimedDataMonitorPanel::ShowEditorPerformanceThrottlingWarning() const
{
	const UEditorPerformanceSettings* Settings = GetDefault<UEditorPerformanceSettings>();
	return Settings->bThrottleCPUWhenNotForeground ? EVisibility::Visible : EVisibility::Collapsed;
}


FReply STimedDataMonitorPanel::DisableEditorPerformanceThrottling()
{
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bThrottleCPUWhenNotForeground = false;
	Settings->PostEditChange();
	Settings->SaveConfig();
	return FReply::Handled();
}


EVisibility STimedDataMonitorPanel::GetThrobberVisibility() const
{
	return MonitorCalibration ? EVisibility::Visible : EVisibility::Collapsed;
}


void STimedDataMonitorPanel::CalibrateWithTimecode()
{
	MessageLogListing->ClearMessages();
	
	FTimedDataMonitorCalibrationParameters Params = GetDefault<UTimedDataMonitorEditorSettings>()->CalibrationSettings;
	FTimedDataMonitorCalibration::FOnCalibrationCompletedSignature CompletedCallback = FTimedDataMonitorCalibration::FOnCalibrationCompletedSignature::CreateSP(this, &STimedDataMonitorPanel::CalibrateWithTimecodeCompleted);

	MonitorCalibration = MakeUnique<FTimedDataMonitorCalibration>();
	MonitorCalibration->CalibrateWithTimecode(Params, CompletedCallback);

	GetMutableDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType = ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode;
	GetMutableDefault<UTimedDataMonitorEditorSettings>()->SaveConfig();
}

void STimedDataMonitorPanel::CalibrateWithTimecodeCompleted(FTimedDataMonitorCalibrationResult Result)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	MonitorCalibration.Reset();
	MarkSettingsDirty();

	if (Result.ReturnCode != ETimedDataMonitorCalibrationReturnCode::Succeeded)
	{
		switch (Result.ReturnCode)
		{
		case ETimedDataMonitorCalibrationReturnCode::Succeeded:
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Info
				, LOCTEXT("CalibrateSucceeded", "Calibration succeeded.")));
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_NoTimecode:
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, LOCTEXT("CalibrateFailed_NoTimecode", "The timecode provider doesn't have a proper timecode value.")));
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_UnresponsiveInput:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("CalibrateFailed_UnresponsiveInput", "The input '{0}' is unresponsive."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_InvalidEvaluationType:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("CalibrationFailed_InvalidEvaluationType", "The input '{0}' is not evaluated in timecode."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_InvalidFrameRate:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("CalibrationFailed_InvalidFrameRate", "The input '{0}' has an invalid frame rate."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_NoDataBuffered:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("CalibrationFailed_NoDataBuffered", "The input '{0}' has at least one channel that does not data buffured."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		case ETimedDataMonitorCalibrationReturnCode::Failed_Reset:
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, LOCTEXT("CalibrationFailed_Reset", "The process was manually interrupted.")));
			break;
		case ETimedDataMonitorCalibrationReturnCode::Retry_NotEnoughData:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("CalibrationFailed_IncreaseBufferSize", "No interval could be found. The buffer size that is required for input '{0}' should be enough. It doesn't contain enough data."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		case ETimedDataMonitorCalibrationReturnCode::Retry_IncreaseBufferSize:
			for (const FTimedDataMonitorInputIdentifier& Identifier : Result.FailureInputIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Warning
					, FText::Format(LOCTEXT("CalibrationRetry_IncreaseBufferSize", "No interval could be found. The buffer size of input '{0}' has been increased. You may retry when the buffer will be full."), TimedDataMonitorSubsystem->GetInputDisplayName(Identifier))));
			}
			break;
		default:
			break;
		}
	}
}


void STimedDataMonitorPanel::ApplyTimeCorrectionAll()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	MessageLogListing->ClearMessages();

	TArray<FTimedDataMonitorInputIdentifier> AllInputs = TimedDataMonitorSubsystem->GetAllInputs();
	bool bAllSucceeded = true;
	for (const FTimedDataMonitorInputIdentifier& InputIndentifier : AllInputs)
	{
		if (TimedDataMonitorSubsystem->GetInputEnabled(InputIndentifier) != ETimedDataMonitorInputEnabled::Disabled)
		{
			ETimedDataMonitorTimeCorrectionReturnCode Result = ApplyTimeCorrection(InputIndentifier);
			bAllSucceeded = bAllSucceeded && (Result == ETimedDataMonitorTimeCorrectionReturnCode::Succeeded);
		}
	}

	if (bAllSucceeded)
	{
		MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Info
			, LOCTEXT("JamSucceeded", "Time Correction succeeded.")));
	}

	MarkSettingsDirty();

	GetMutableDefault<UTimedDataMonitorEditorSettings>()->LastCalibrationType = ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode;
	GetMutableDefault<UTimedDataMonitorEditorSettings>()->SaveConfig();
}


ETimedDataMonitorTimeCorrectionReturnCode STimedDataMonitorPanel::ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& InputIdentifier)
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	const FTimedDataMonitorTimeCorrectionResult Result = FTimedDataMonitorCalibration::ApplyTimeCorrection(InputIdentifier, GetDefault<UTimedDataMonitorEditorSettings>()->TimeCorrectionSettings);
	switch (Result.ReturnCode)
	{
	case ETimedDataMonitorTimeCorrectionReturnCode::Failed_InvalidInput:
		MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
			, LOCTEXT("JamFailed_InvalidInput", "The input identifier was invalid.")));
		break;
	case ETimedDataMonitorTimeCorrectionReturnCode::Failed_NoTimecode:
		MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
			, LOCTEXT("JamFailed_NoTimecode", "The timecode provider doesn't have a proper timecode value.")));
		break;
	case ETimedDataMonitorTimeCorrectionReturnCode::Failed_UnresponsiveInput:
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Result.FailureChannelIdentifiers)
		{
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, FText::Format(LOCTEXT("JamFailed_UnresponsiveInput", "The channel '{0}' is unresponsive."), TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier))));
		}
		break;
	case ETimedDataMonitorTimeCorrectionReturnCode::Failed_NoDataBuffered:
		for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Result.FailureChannelIdentifiers)
		{
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, FText::Format(LOCTEXT("JamFailed_NoDataBuffered", "The channel '{0}' has not data buffered."), TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier))));
		}
		break;
	case ETimedDataMonitorTimeCorrectionReturnCode::Failed_BufferCouldNotBeResize:
		if (Result.FailureChannelIdentifiers.Num() > 0)
		{
			for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Result.FailureChannelIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("JamFailed_BufferSizeHaveBeenMaxed_Channel", "The buffer size of channel '{0}' could not be increased."), TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier))));
			}
		}
		else
		{
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, FText::Format(LOCTEXT("JamFailed_BufferSizeHaveBeenMaxed_Input", "The buffer size of input '{0}' could not be increased."), TimedDataMonitorSubsystem->GetInputDisplayName(InputIdentifier))));
		}
		break;
	case ETimedDataMonitorTimeCorrectionReturnCode::Retry_NotEnoughData:
		if (Result.FailureChannelIdentifiers.Num() > 0)
		{
			for (const FTimedDataMonitorChannelIdentifier& ChannelIdentifier : Result.FailureChannelIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
					, FText::Format(LOCTEXT("JamRetry_IncreaseBufferSize_Channel", "No interval could be found. The buffer  that is required for channel '{0}' could not be increased."), TimedDataMonitorSubsystem->GetChannelDisplayName(ChannelIdentifier))));
			}
		}
		else
		{
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, FText::Format(LOCTEXT("JamRetry_IncreaseBufferSize_Input", "No interval could be found. The buffer size that is required for input '{0}' should be enough. It doesn't contain enough data."), TimedDataMonitorSubsystem->GetInputDisplayName(InputIdentifier))));
		}
		break;				
	case ETimedDataMonitorTimeCorrectionReturnCode::Retry_IncreaseBufferSize:
		if (Result.FailureChannelIdentifiers.Num() > 0)
		{
			for (const FTimedDataMonitorChannelIdentifier& Identifier : Result.FailureChannelIdentifiers)
			{
				MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Warning
					, FText::Format(LOCTEXT("JamRetry_BufferSizeHasBeenIncrease_Channel", "The buffer size of channel '{0}' needed to be increased. Retry."), TimedDataMonitorSubsystem->GetChannelDisplayName(Identifier))));
			}
		}
		else
		{
			MessageLogListing->AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error
				, FText::Format(LOCTEXT("JamRetry_BufferSizeHasBeenIncrease_Input", "The buffer size of input '{0}' needed to be increased. Retry."), TimedDataMonitorSubsystem->GetInputDisplayName(InputIdentifier))));
		}
		break;
	default:
		break;
	}

	return Result.ReturnCode;
}


FReply STimedDataMonitorPanel::OnCancelCalibration()
{
	if (MonitorCalibration)
	{
		MonitorCalibration.Reset();
	}
	return FReply::Handled();
}

FLiveLinkClient* STimedDataMonitorPanel::GetLiveLinkClient() const
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		return static_cast<FLiveLinkClient*>(&IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName));
	}
	return nullptr;
}

namespace SavePresetUtils
{
	static bool OpenSaveDialog(const FString& InDefaultPath, const FString& InNewNameSuggestion, FString& OutPackageName)
	{
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InNewNameSuggestion;
		SaveAssetDialogConfig.AssetClassNames.Add(ULiveLinkPreset::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveLiveLinkPresetDialogTitle", "Save Live Link Preset");

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

		if (!SaveObjectPath.IsEmpty())
		{
			OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			return true;
		}
		return false;
	}

	static bool GetSavePresetPackageName(FString& OutName)
	{
		ULiveLinkUserSettings* ConfigSettings = GetMutableDefault<ULiveLinkUserSettings>();

		FDateTime Today = FDateTime::Now();
		TMap<FString, FStringFormatArg> FormatArgs;
		FormatArgs.Add(TEXT("date"), Today.ToString(TEXT("%Y-%m-%d-%H%M%S")));

		const FString DefaultSaveDirectory = FString::Format(*ConfigSettings->GetPresetSaveDir().Path, FormatArgs);

		FString DialogStartPath;

		FPackageName::TryConvertFilenameToLongPackageName(DefaultSaveDirectory, DialogStartPath);

		if (DialogStartPath.IsEmpty())
		{
			DialogStartPath = TEXT("/Game");
		}

		FString DefaultName = LOCTEXT("NewLiveLinkPreset", "NewLiveLinkPreset").ToString();

		FString UniquePackageName;
		FString UniqueAssetName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		AssetToolsModule.Get().CreateUniqueAssetName(DialogStartPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);

		FString DialogStartName = FPaths::GetCleanFilename(UniqueAssetName);
		FString UserPackageName;
		FString NewPackageName;

		bool bFilenameValid = false;

		while (!bFilenameValid)
		{
			if (!OpenSaveDialog(DialogStartPath, DialogStartName, UserPackageName))
			{
				return false;
			}

			NewPackageName = UserPackageName;

			FText OutError;
			bFilenameValid = FPackageName::IsValidLongPackageName(NewPackageName, false, &OutError);
		}

		ConfigSettings->PresetSaveDir.Path = FPackageName::GetLongPackagePath(UserPackageName);
		ConfigSettings->SaveConfig();

		OutName = MoveTemp(NewPackageName);

		return true;
	}
}

void STimedDataMonitorPanel::CollectInputSettingsPackages(TArray<UPackage*>& OutPackages)
{
	// If the timecode provider is an asset sub-object (e.g. from a media profile), include its package.
	if (UTimecodeProvider* TimecodeProvider = GEngine->GetTimecodeProvider())
	{
		if (TimecodeProvider->GetOuter() != GEngine)
		{
			UPackage* Package = TimecodeProvider->GetPackage();
			if (Package && Package != GetTransientPackage())
			{
				TimecodeProvider->MarkPackageDirty();
				OutPackages.AddUnique(Package);
			}
		}
	}

	UTimedDataMonitorSubsystem* Subsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();

	if (!Subsystem)
	{
		return;
	}

	TArray<FTimedDataMonitorInputIdentifier> AllInputs = Subsystem->GetAllInputs();

	for (const FTimedDataMonitorInputIdentifier& InputId : AllInputs)
	{
		ITimedDataInput* Input = Subsystem->GetTimedDataInput(InputId);

		if (!Input)
		{
			continue;
		}

		UObject* SettingsObject = Input->GetSettingsObject();

		if (!SettingsObject)
		{
			continue;
		}

		UPackage* Package = SettingsObject->GetPackage();

		if (Package && Package != GetTransientPackage())
		{
			SettingsObject->MarkPackageDirty();
			OutPackages.AddUnique(Package);
		}
	}
}

const FSlateBrush* STimedDataMonitorPanel::GetSaveButtonBrush() const
{
	return FAppStyle::GetBrush(bSettingsDirty ? TEXT("Icons.SaveChanged") : TEXT("Icons.Save"));
}

FReply STimedDataMonitorPanel::OnSavePreset()
{
	FLiveLinkClient* LiveLinkClient = GetLiveLinkClient();
	const bool bHasLiveLinkSources = LiveLinkClient && !LiveLinkClient->GetSources().IsEmpty();

	// If LL sources exist but no preset has been saved yet, prompt for a name.
	if (bHasLiveLinkSources && !LiveLinkClient->GetLastSavedPreset())
	{
		OnSaveAsPreset();
		return FReply::Handled();
	}

	TArray<UPackage*> PackagesToSave;

	if (bHasLiveLinkSources)
	{
		ULiveLinkPreset* CurrentPreset = LiveLinkClient->GetLastSavedPreset();
		FScopedTransaction Transaction(LOCTEXT("SavePreset", "Save Preset"));
		CurrentPreset->BuildFromClient();
		CurrentPreset->MarkPackageDirty();
		PackagesToSave.Add(CurrentPreset->GetPackage());
	}

	CollectInputSettingsPackages(PackagesToSave);

	if (!PackagesToSave.IsEmpty())
	{
		const FEditorFileUtils::EPromptReturnCode Result = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
		if (Result == FEditorFileUtils::PR_Success)
		{
			bSettingsDirty = false;
		}
	}
	else
	{
		bSettingsDirty = false;
	}

	return FReply::Handled();
}

void STimedDataMonitorPanel::OnSaveAsPreset()
{
	FString PackageName;
	if (!SavePresetUtils::GetSavePresetPackageName(PackageName))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SaveAsPreset", "Save As Preset"));

	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* NewPackage = CreatePackage(*PackageName);
	ULiveLinkPreset* NewPreset = NewObject<ULiveLinkPreset>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (NewPreset)
	{
		TArray<UPackage*> PackagesToSave;

		NewPreset->BuildFromClient();
		NewPreset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(NewPreset);
		PackagesToSave.Add(NewPackage);

		CollectInputSettingsPackages(PackagesToSave);

		const FEditorFileUtils::EPromptReturnCode Result = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
		if (Result == FEditorFileUtils::PR_Success)
		{
			bSettingsDirty = false;

			FLiveLinkClient* LiveLinkClient = GetLiveLinkClient();
			if (LiveLinkClient)
			{
				LiveLinkClient->SetLastSavedPreset(NewPreset);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
