// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Widgets/SCompoundWidget.h"

#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "TimedDataMonitorEditorSettings.h"

enum class ETimedDataMonitorTimeCorrectionReturnCode : uint8;

struct FSlateBrush;
struct FTimedDataMonitorCalibrationResult;

class FLiveLinkClient;
class FTabManager;
class FTimedDataMonitorCalibration;
class FWorkspaceItem;
class IMessageLogListing;
class STimedDataGenlock;
class STimedDataInputListView;
class STimedDataTimecodeProvider;
class SWidget;
class ULiveLinkPreset;

enum class ETimedDataMonitorEvaluationState : uint8;

class STimedDataMonitorPanel : public SCompoundWidget
{
public:
	static void RegisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager, TSharedRef<FWorkspaceItem> InWorkspaceItem);
	static void UnregisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager);
	static TSharedPtr<STimedDataMonitorPanel> GetPanelInstance();
	static const FName TabName;

private:
	using Super = SCompoundWidget;
	static TWeakPtr<STimedDataMonitorPanel> WidgetInstance;

public:
	SLATE_BEGIN_ARGS(STimedDataMonitorPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void RequestRefresh() { bRefreshRequested = true; }

	/** Flags that monitored input settings have been modified and need saving. */
	void MarkSettingsDirty()
	{
		bSettingsDirty = true;
	}

private:
	FReply OnCalibrateClicked();
	TSharedRef<SWidget> OnCalibrateBuildMenu();
	FText GetCalibrateButtonTooltip() const;
	const FSlateBrush* GetCalibrateButtonImage() const;
	FText GetCalibrateButtonText() const;
	FReply OnResetErrorsClicked();
	TSharedRef<SWidget> OnResetBuildMenu();
	void OnResetBufferStatClicked();
	bool IsResetBufferStatChecked() const;
	void OnClearMessageClicked();
	bool IsClearMessageChecked() const;
	void OnResetAllEvaluationTimeClicked();
	bool IsResetEvaluationChecked() const;
	FReply OnShowBuffersClicked();
	FReply OnGeneralUserSettingsClicked();
	FSlateColor GetEvaluationStateColorAndOpacity() const;
	FText GetEvaluationStateText() const;
	TSharedPtr<SWidget> OnDataListConstructContextMenu();
	bool IsSourceListSectionValid() const;
	void ApplyTimeCorrectionOnSelection();
	void ResetTimeCorrectionOnSelection();

	EVisibility ShowMessageLog() const;
	EVisibility ShowEditorPerformanceThrottlingWarning() const;
	FReply DisableEditorPerformanceThrottling();

	EVisibility GetThrobberVisibility() const;

	void BuildCalibrationArray();
	void CalibrateWithTimecode();
	void CalibrateWithTimecodeCompleted(FTimedDataMonitorCalibrationResult);
	void ApplyTimeCorrectionAll();
	ETimedDataMonitorTimeCorrectionReturnCode ApplyTimeCorrection(const FTimedDataMonitorInputIdentifier& InputIndentifier);
	FReply OnCancelCalibration();

	/** Saves the tracked preset and modified input settings, or falls back to Save As if no preset is tracked. */
	FReply OnSavePreset();
	/** Opens a Save As dialog to create a new Live Link preset and saves all input settings. */
	void OnSaveAsPreset();
	/** Collects packages from all monitored inputs that have a settings object (e.g. media sources). */
	void CollectInputSettingsPackages(TArray<UPackage*>& OutPackages);
	/** Returns Icons.SaveChanged when dirty, Icons.Save otherwise. */
	const FSlateBrush* GetSaveButtonBrush() const;
	/** Returns the Live Link client from modular features, or nullptr. */
	FLiveLinkClient* GetLiveLinkClient() const;

private:
	TSharedPtr<STimedDataGenlock> TimedDataGenlockWidget;
	TSharedPtr<STimedDataTimecodeProvider> TimedDataTimecodeWidget;
	TSharedPtr<STimedDataInputListView> TimedDataSourceList;
	TSharedPtr<IMessageLogListing> MessageLogListing;

	static const int32 CalibrationArrayCount = (int32)ETimedDataMonitorEditorCalibrationType::Max;
	FUIAction CalibrationUIAction[CalibrationArrayCount];
	FSlateIcon CalibrationSlateIcon[CalibrationArrayCount];
	FText CalibrationName[CalibrationArrayCount];
	FText CalibrationTooltip[CalibrationArrayCount];

	ETimedDataMonitorEvaluationState CachedGlobalEvaluationState;

	TUniquePtr<FTimedDataMonitorCalibration> MonitorCalibration;

	bool bRefreshRequested = true;
	/** True when monitored input settings have been modified since last save. */
	bool bSettingsDirty = false;
	double LastCachedValueUpdateTime = 0.0;
};
