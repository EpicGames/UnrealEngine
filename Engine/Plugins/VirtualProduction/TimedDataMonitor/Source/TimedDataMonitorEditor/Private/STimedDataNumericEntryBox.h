// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "TimedDataMonitorEditorStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "TimeDataNumericEntryBox"

template<class NumericType>
class STimedDataNumericEntryBox : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;
	using FOnValueCommitted = typename SNumericEntryBox<NumericType>::FOnValueCommitted;
	
public:
	SLATE_BEGIN_ARGS(STimedDataNumericEntryBox)
			: _Value(0)
			, _SuffixWidget(SNullWidget::NullWidget)
			, _CanEdit(true)
			, _Amount(0)
			, _ShowAmount(false)
			, _TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")))
			, _ComboButton(true)
			, _ControlModifierOffset(10)
	{
		_ShiftModifierOffset = NumericType(1) / 10;
		if (!_ShiftModifierOffset)
		{
			_ShiftModifierOffset = 1;
		}

		_AltModifierOffset = NumericType(1) / 100;
		if (!_AltModifierOffset)
		{
			_AltModifierOffset = 1;
		}
	}
		SLATE_ATTRIBUTE(NumericType, Value)
		SLATE_ARGUMENT(TOptional<NumericType>, MinValue)
		SLATE_ARGUMENT(TOptional<NumericType>, MaxValue)
		SLATE_ARGUMENT(FText, EditLabel)
		SLATE_ARGUMENT(TSharedRef<SWidget>, SuffixWidget)
		SLATE_ARGUMENT(bool, CanEdit);
		SLATE_ATTRIBUTE(NumericType, Amount);
		SLATE_ARGUMENT(bool, ShowAmount);
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(bool, ComboButton)
		SLATE_EVENT(FOnValueCommitted, OnValueCommitted)
		SLATE_ARGUMENT(NumericType, ControlModifierOffset)
		SLATE_ARGUMENT(NumericType, AltModifierOffset)
		SLATE_ARGUMENT(NumericType, ShiftModifierOffset)
		SLATE_ATTRIBUTE(FText, MinusButtonToolTipText)
		SLATE_ATTRIBUTE(FText, PlusButtonToolTipText)
	SLATE_END_ARGS()
	
public:
	void Construct(const FArguments& InArgs)
	{
		Value = InArgs._Value;
		CachedValue = Value.Get();
		MinValue = InArgs._MinValue;
		MaxValue = InArgs._MaxValue;
		SuffixWidget = InArgs._SuffixWidget;
		Amount = InArgs._Amount;
		bShowAmount = InArgs._ShowAmount;
		bComboButton = InArgs._ComboButton;
		OnValueCommitted = InArgs._OnValueCommitted;
		AltOffset = InArgs._AltModifierOffset;
		ControlOffset = InArgs._ControlModifierOffset;
		ShiftOffset = InArgs._ShiftModifierOffset;
		MinusButtonToolTip = InArgs._MinusButtonToolTipText;
		PlusButtonToolTip = InArgs._PlusButtonToolTipText;

		TSharedRef<SWidget> TextBlock = SNew(STextBlock)
			.TextStyle(InArgs._TextStyle)
			.Text(this, &STimedDataNumericEntryBox::GetValueText);

		if (InArgs._ComboButton)
		{
			if (InArgs._CanEdit)
			{
				ChildSlot
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.FillWidth(1.f)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						TextBlock
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SAssignNew(ComboButton, SComboButton)
						.ComboButtonStyle(FTimedDataMonitorEditorStyle::Get(), "ToggleComboButton")
						.HAlign(HAlign_Center)
						.HasDownArrow(false)
						.OnGetMenuContent(this, &STimedDataNumericEntryBox::OnCreateEditMenu)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
							.Text(FEditorFontGlyphs::Pencil_Square)
							.ColorAndOpacity(FLinearColor::White)
						]
					]
				];
			}
			else
			{
				ChildSlot
				[
					TextBlock
				];
			}
		}
		else
		{

			if (InArgs._CanEdit)
			{
				ChildSlot
				[
					OnCreateEditMenu()
				];
			}
			else
			{
				ChildSlot
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FTimedDataMonitorEditorStyle::Get(), "TextBlock.Regular")
						.Text(InArgs._EditLabel)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						TextBlock
					]
				];
			}
		}
	}

private:
	FText GetValueText() const 
	{
		if (bShowAmount)
		{
			return FText::Format(NSLOCTEXT("TimedDataNumericEntryBox", "TimedNumericDataState", "{0}/{1}"), Amount.Get(), Value.Get());
		}
		else
		{
			return FText::AsNumber(Value.Get()); 
		}
	}

	TSharedRef<SWidget> OnCreateEditMenu()
	{
		TSharedRef<SWidget> InnerWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(2.0f, 0.f)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "FlatButton")
				.ToolTipText(MinusButtonToolTip)
				.OnClicked(this, &STimedDataNumericEntryBox::OnMinusClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FTimedDataMonitorEditorStyle::Get().GetBrush("MinusButton"))
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(EntryBox, SNumericEntryBox<NumericType>)
				.MinValue(MinValue)
				.MaxValue(MaxValue)
				.MinDesiredValueWidth(50)
				.Value_Lambda([this]() 
				{
					CachedValue = Value.Get();
					return TOptional<NumericType>(Value.Get()); 
				})
				.OnValueCommitted(this, &STimedDataNumericEntryBox::OnValueCommittedCallback)

			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FTimedDataMonitorEditorStyle::Get(), "FlatButton")
				.ToolTipText(PlusButtonToolTip)
				.OnClicked(this, &STimedDataNumericEntryBox::OnPlusClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FTimedDataMonitorEditorStyle::Get().GetBrush("PlusButton"))
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SuffixWidget
			];


		if (bComboButton)
		{
			EditMenuContent = SNew(SBorder)
				.VAlign(VAlign_Center)
				.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
				[
					InnerWidget
				];
		}
		else
		{
			EditMenuContent = InnerWidget;
		}
		
		return EditMenuContent.ToSharedRef();
	}

	void CloseComboButton(ECheckBoxState InNewState)
	{
		if (ComboButton)
		{
			ComboButton->SetIsOpen(false);
		}
	}

	void OnValueCommittedCallback(NumericType NewValue, ETextCommit::Type CommitType)
	{
		if (MinValue.IsSet())
		{
			NewValue = FMath::Max(MinValue.GetValue(), NewValue);
		}

		if (MaxValue.IsSet())
		{
			NewValue = FMath::Min(MaxValue.GetValue(), NewValue);
		}

		CachedValue = NewValue;
		OnValueCommitted.ExecuteIfBound(NewValue, CommitType);
		if (!ComboButton && CommitType == ETextCommit::Type::OnEnter)
		{
			// Clear focus to get the updated value in case it was modified externally.
			FSlateApplication::Get().ClearKeyboardFocus();
			FTimerHandle Handle;
			constexpr float InRate = 0.01f;
			constexpr bool InbLoop = false;
			GEditor->GetTimerManager()->SetTimer(Handle, [this](){ FSlateApplication::Get().SetKeyboardFocus(EntryBox); }, InRate, InbLoop);
		}
	}

	FReply OnMinusClicked()
	{
		NumericType Offset = GetModifierKeyOffset();
		if (EntryBox)
		{
			CachedValue = Value.Get();
			CachedValue -= Offset;

			if (MinValue.IsSet() && CachedValue - Offset < MinValue.GetValue())
			{
				CachedValue = MinValue.GetValue();
			}
			
			OnValueCommitted.ExecuteIfBound(CachedValue, ETextCommit::Type::Default);
		}
		return FReply::Handled();
	}

	FReply OnPlusClicked()
	{
		NumericType Offset = GetModifierKeyOffset();
		if (EntryBox)
		{
			CachedValue = Value.Get();
			CachedValue += Offset;

			if (MaxValue.IsSet() && CachedValue + Offset > MaxValue.GetValue())
			{
				CachedValue = MaxValue.GetValue();
			}
			OnValueCommitted.ExecuteIfBound(CachedValue, ETextCommit::Type::Default);
		}
		return FReply::Handled();
	}

	NumericType GetModifierKeyOffset() const
	{
		const bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		const bool bIsControlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();
		const bool bIsAltDown = FSlateApplication::Get().GetModifierKeys().IsAltDown();
		
		NumericType Offset = 1;
		
		if (bIsShiftDown)
		{
			Offset = ShiftOffset;
		}
		else if (bIsControlDown)
		{
			Offset = ControlOffset;
		}
		else if (bIsAltDown)
		{
			Offset = AltOffset;
		}

		return Offset;
	}

private:
	TAttribute<NumericType> Value;
	TAttribute<NumericType> Amount;
	NumericType CachedValue;
	TSharedPtr<SNumericEntryBox<NumericType>> EntryBox;
	TOptional<NumericType> MinValue;
	TOptional<NumericType> MaxValue;
	FText EditLabel;
	TSharedRef<SWidget> SuffixWidget = SNullWidget::NullWidget;
	bool bShowAmount;
	FOnValueCommitted OnValueCommitted;
	TSharedPtr<SComboButton> ComboButton;
	bool bComboButton = true;
	bool bCloseRequested = false;
	TSharedPtr<SWidget> EditMenuContent;
	NumericType AltOffset;
	NumericType ShiftOffset;
	NumericType ControlOffset;
	TAttribute<FText> MinusButtonToolTip;
	TAttribute<FText> PlusButtonToolTip;
};


#undef LOCTEXT_NAMESPACE
