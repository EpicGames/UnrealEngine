// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimedDataTimecodeProvider.h"

#include "Engine/TimecodeProvider.h"
#include "ISettingsModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"


#include "STimedDataMonitorPanel.h"
#include "STimedDataNumericEntryBox.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "STimedDataTimecodeProvider"


namespace TimedDataTimecodeProvider
{
	FTimespan FromPlatformSeconds(double InPlatformSeconds)
	{
		const FDateTime NowDateTime = FDateTime::Now();
		const double HighPerformanceClock = FPlatformTime::Seconds();
		const double DateTimeSeconds = (InPlatformSeconds - HighPerformanceClock) + NowDateTime.GetTimeOfDay().GetTotalSeconds();
		return FTimespan::FromSeconds(DateTimeSeconds);
	}
}

void STimedDataTimecodeProvider::Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> InOwnerPanel)
{
	OwnerPanel = InOwnerPanel;

	UpdateCachedValue();

	FSlateFontInfo TimeFont = FCoreStyle::Get().GetFontStyle(TEXT("EmbossedText"));
	TimeFont.Size += 2;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(this, &STimedDataTimecodeProvider::GetStateText)
				.ColorAndOpacity(this, &STimedDataTimecodeProvider::GetStateColorAndOpacity)
			]
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
				.Text(this, &STimedDataTimecodeProvider::GetTimecodeProviderText)
			]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Padding(4.f)
				.ToolTipText(LOCTEXT("ShowTimecodeProviderSetting_Tip", "Show timecode provider setting"))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled(this, &STimedDataTimecodeProvider::IsTimecodeOffsetEnabled)
				.IsChecked_Lambda([]() {return ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &STimedDataTimecodeProvider::ShowTimecodeProviderSetting)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Cogs)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				SNew(SCheckBox)
				.Padding(4.f)
				.ToolTipText(LOCTEXT("ReapplyMenuToolTip", "Reinitialize the current Timecode Provider."))
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled(this, &STimedDataTimecodeProvider::IsTimecodeOffsetEnabled)
				.IsChecked_Lambda([]() {return ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &STimedDataTimecodeProvider::ReinitializeTimecodeProvider)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Undo)
				]
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.f,4.f, 6.f, 4.f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(TimeFont)
					.Text(this, &STimedDataTimecodeProvider::GetTimecodeText)
				]
				//+ SVerticalBox::Slot()
				//[
				//	SNew(STextBlock)
				//	.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
				//	.Text(this, &STimedDataTimecodeProvider::GetSystemTimeText)
				//]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalText")))
				.Text(LOCTEXT("GlobalFrameOffsetLabel", "Global Frame Offset: "))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STimedDataNumericEntryBox<float>)
				.ComboButton(false)
				.ToolTipText(LOCTEXT("FrameDelay_ToolTip", "Number of frames to subtract from the original timecode."))
				.Value(this, &STimedDataTimecodeProvider::GetTimecodeFrameDelay)
				.MinValue(TOptional<float>())
				.MaxValue(TOptional<float>())
				.OnValueCommitted(this, &STimedDataTimecodeProvider::SetTimecodeFrameDelay)
				.IsEnabled(this, &STimedDataTimecodeProvider::IsTimecodeOffsetEnabled)
			]
		]
	];

	SetToolTip(SNew(SToolTip).Text(this, &STimedDataTimecodeProvider::GetTooltipText));
}


void STimedDataTimecodeProvider::RequestRefresh()
{
	if (TSharedPtr<STimedDataMonitorPanel> OwnerPanelPin = OwnerPanel.Pin())
	{
		OwnerPanelPin->RequestRefresh();
	}
}


void STimedDataTimecodeProvider::UpdateCachedValue()
{
	CachedPlatformSeconds = FApp::GetCurrentTime();
	FTimespan PlatformSecond = TimedDataTimecodeProvider::FromPlatformSeconds(CachedPlatformSeconds);
	CachedSystemTime = FText::FromString(PlatformSecond.ToString());

	//Cache info about the current timecode provider
	if (const UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		CachedState = TimecodeProviderPtr->GetSynchronizationState();
		CachedTimecodeProvider = FText::FromName(TimecodeProviderPtr->GetFName());
	}
	else
	{
		CachedState = ETimecodeProviderSynchronizationState::Closed;
		CachedTimecodeProvider = LOCTEXT("Undefined", "<Undefined>");
	}

	//Cache info about current timecode
	CachedFrameTimeOptional = FApp::GetCurrentFrameTime();
	if (CachedFrameTimeOptional.IsSet())
	{
		CachedTimecode = FText::Format(LOCTEXT("TimecodeFormat", "{0} @ {1}")
			, FText::FromString(CachedFrameTimeOptional->ToTimecode().ToString())
			, CachedFrameTimeOptional->Rate.ToPrettyText());
	}
	else
	{
		CachedTimecode = LOCTEXT("None", "<None>");
	}
}


FText STimedDataTimecodeProvider::GetStateText() const
{
	switch (CachedState)
	{
		case ETimecodeProviderSynchronizationState::Synchronized:
			return FEditorFontGlyphs::Film;
		case ETimecodeProviderSynchronizationState::Synchronizing:
			return FEditorFontGlyphs::Hourglass_O;
		case ETimecodeProviderSynchronizationState::Error:
		case ETimecodeProviderSynchronizationState::Closed:
			return FEditorFontGlyphs::Ban;
	}
	return FEditorFontGlyphs::Exclamation;
}


FSlateColor STimedDataTimecodeProvider::GetStateColorAndOpacity() const
{
	switch (CachedState)
	{
	case ETimecodeProviderSynchronizationState::Closed:
	case ETimecodeProviderSynchronizationState::Error:
		return FLinearColor::Red;
	case ETimecodeProviderSynchronizationState::Synchronized:
		return FLinearColor::Green;
	case ETimecodeProviderSynchronizationState::Synchronizing:
		return FLinearColor::Yellow;
	}

	return FLinearColor::Red;
}


FText STimedDataTimecodeProvider::GetTooltipText() const
{
	return FText::Format(LOCTEXT("TimedDataTimecodeProviderTooltip", "TC Provider: {0}\nTC: {1}\nTC FrameRate: {2}\nFrame Number: {3}\nSystemTime: {4}\nPlatformTime: {5}")
		, CachedTimecodeProvider
		, CachedTimecode
		, CachedFrameTimeOptional.IsSet() ? CachedFrameTimeOptional->Rate.ToPrettyText() : FText::GetEmpty()
		, CachedFrameTimeOptional.IsSet() ? FText::AsNumber(CachedFrameTimeOptional->Time.AsDecimal()) : FText::GetEmpty()
		, CachedSystemTime
		, FText::AsNumber(CachedPlatformSeconds));
}


float STimedDataTimecodeProvider::GetTimecodeFrameDelay() const
{
	if (const UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		return TimecodeProviderPtr->FrameDelay;
	}
	return 0.f;
}


void STimedDataTimecodeProvider::SetTimecodeFrameDelay(float Value, ETextCommit::Type)
{
	if (UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		if (TimecodeProviderPtr->FrameDelay != Value)
		{
			FObjectEditorUtils::SetPropertyValue(TimecodeProviderPtr, GET_MEMBER_NAME_CHECKED(UTimecodeProvider, FrameDelay), Value);
			RequestRefresh();
		}
	}
}


bool STimedDataTimecodeProvider::IsTimecodeOffsetEnabled() const
{
	return GEngine && GEngine->GetTimecodeProvider() != nullptr;
}


void STimedDataTimecodeProvider::ShowTimecodeProviderSetting(ECheckBoxState)
{
	if (UTimecodeProvider* TimecodeProviderPtr = GEngine->GetTimecodeProvider())
	{
		if (TimecodeProviderPtr->GetOuter() == GEngine)
		{
			FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Engine", "General");
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TimecodeProviderPtr->GetOuter());
		}
	}
}


void STimedDataTimecodeProvider::ReinitializeTimecodeProvider(ECheckBoxState)
{
	if (GEngine)
	{
		GEngine->ReinitializeTimecodeProvider();
	}
}


#undef LOCTEXT_NAMESPACE
