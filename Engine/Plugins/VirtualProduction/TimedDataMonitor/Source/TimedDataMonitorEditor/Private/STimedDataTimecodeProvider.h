// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Misc/QualifiedFrameTime.h"

enum class ETimecodeProviderSynchronizationState;
namespace ETextCommit { enum Type : int; }


class STimedDataMonitorPanel;
enum class ECheckBoxState : uint8;

class STimedDataTimecodeProvider : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimedDataTimecodeProvider) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<STimedDataMonitorPanel> OwnerPanel);

	void RequestRefresh();
	void UpdateCachedValue();

private:

	FText GetStateText() const;
	FSlateColor GetStateColorAndOpacity() const;
	FText GetTimecodeProviderText() const { return CachedTimecodeProvider; }
	FText GetTimecodeText() const { return CachedTimecode; }
	FText GetSystemTimeText() const { return CachedSystemTime; }
	FText GetTooltipText() const;

	float GetTimecodeFrameDelay() const;
	void SetTimecodeFrameDelay(float Offset, ETextCommit::Type);
	bool IsTimecodeOffsetEnabled() const;

	void ShowTimecodeProviderSetting(ECheckBoxState);
	void ReinitializeTimecodeProvider(ECheckBoxState);

private:
	TWeakPtr<STimedDataMonitorPanel> OwnerPanel;

	FText CachedTimecode;
	FText CachedSystemTime;
	FText CachedTimecodeProvider;
	TOptional<FQualifiedFrameTime> CachedFrameTimeOptional;
	double CachedPlatformSeconds;
	ETimecodeProviderSynchronizationState CachedState;
};
