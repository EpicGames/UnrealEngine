// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingDiagramWidget.h"

#include "Engine/Engine.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "TimedDataMonitorEditorSettings.h"
#include "TimedDataMonitorSubsystem.h"

#include "EditorFontGlyphs.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "STimingDiagramWidget"


/* STimingDiagramWidgetGraphic
 *****************************************************************************/
class STimingDiagramWidgetGraphic : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(STimingDiagramWidgetGraphic)
		: _ShowFurther(false)
		, _ShowMean(true)
		, _ShowSigma(true)
		, _ShowSnapshot(true)
		, _UseNiceBrush(true)
		, _SizePerSeconds(100.f)
		{}
		SLATE_ARGUMENT(bool, ShowFurther)
		SLATE_ARGUMENT(bool, ShowMean)
		SLATE_ARGUMENT(bool, ShowSigma)
		SLATE_ARGUMENT(bool, ShowSnapshot)
		SLATE_ARGUMENT(bool, UseNiceBrush)
		SLATE_ATTRIBUTE(float, SizePerSeconds)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		bShowFurther = InArgs._ShowFurther;
		bShowMean = InArgs._ShowMean;
		bShowSigma = InArgs._ShowSigma;
		bShowSnapshot = InArgs._ShowSnapshot;
		SizePerSecondsAttibute = InArgs._SizePerSeconds;

		NumberOfSigma = GetDefault<UTimedDataMonitorEditorSettings>()->GetNumberOfStandardDeviationForUI();
		bShowSigma = bShowSigma && (NumberOfSigma > 0);

		FontInfo = FAppStyle::Get().GetFontStyle("FontAwesome.11");

		if (InArgs._UseNiceBrush)
		{
			DarkBrush = &FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageFocused;
			BrightBrush = &FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageNormal;
		}
		else
		{
			DarkBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");
			BrightBrush = DarkBrush;
		}
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		LayerId++;

		const float SizeX = AllottedGeometry.GetLocalSize().X;
		const float SizeY = AllottedGeometry.GetLocalSize().Y;

		const float SizeOfFurthur = bShowFurther ? 20.f : 0.f;
		const float LocationOfCenter = FMath::Max((SizeX - SizeOfFurthur - SizeOfFurthur) * 0.5f, 0.f);
		const float SizePerSeconds = SizePerSecondsAttibute.Get();
		const float SizeOfBoxY = (bShowMean && bShowSnapshot) ? SizeY / 2.f : SizeY - 2.f;
		const float LocationOfSnapshotBoxY = (bShowMean && bShowSnapshot) ? 0.f : 2.f;
		const float LocationOfMeanBoxY = (bShowMean && bShowSnapshot) ? SizeY / 2.f : 2.f;
		const float LocationOfFurtherY = (bShowMean && bShowSnapshot) ? 0: 4.f;

		const float SnapshotLocationMinX = LocationOfCenter - ((EvaluationTime - MinSampleTime) * SizePerSeconds);
		const float SnapshotLocationMaxX = LocationOfCenter + ((MaxSampleTime - EvaluationTime) * SizePerSeconds);
		const float MeanLocationMinX = LocationOfCenter - (OldestMean * SizePerSeconds);
		const float MeanLocationMaxX = LocationOfCenter + (NewestMean * SizePerSeconds);

		const bool bDrawFrameTimes = GetDefault<UTimedDataMonitorEditorSettings>()->bDrawFrameTimesInBufferVisualization;
		const bool bUseAccurateFrameTimes = GetDefault<UTimedDataMonitorEditorSettings>()->bUseAccurateFrameTimesInBufferVisualization;

		// square to show that the data goes further
		if (bShowFurther)
		{
			if (bShowSnapshot && SnapshotLocationMinX < SizeOfFurthur)
			{
				FSlateDrawElement::MakeText(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(SizeOfFurthur, SizeOfBoxY), FSlateLayoutTransform(FVector2D(0, LocationOfSnapshotBoxY))),
					FEditorFontGlyphs::Angle_Double_Left,
					FontInfo, ESlateDrawEffect::None,
					FLinearColor::White);
			}

			if (bShowMean && MeanLocationMinX < SizeOfFurthur)
			{
				FSlateDrawElement::MakeText(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(SizeOfFurthur, SizeOfBoxY), FSlateLayoutTransform(FVector2D(0, LocationOfMeanBoxY))),
					FEditorFontGlyphs::Angle_Double_Left,
					FontInfo, ESlateDrawEffect::None,
					FLinearColor::White);
			}
		}

		// data in relation with evaluation time
		if (bShowSnapshot)
		{
			const float DrawLocationX = FMath::Max(SnapshotLocationMinX, SizeOfFurthur);
			if (DrawLocationX < SizeX - SizeOfFurthur && SnapshotLocationMaxX > SizeOfFurthur)
			{
				const float DrawSizeX = FMath::Clamp(SnapshotLocationMaxX - DrawLocationX, 1.f, SizeX - DrawLocationX - SizeOfFurthur);
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(DrawSizeX, SizeOfBoxY), FSlateLayoutTransform(FVector2D(DrawLocationX, LocationOfSnapshotBoxY))),
					DarkBrush, ESlateDrawEffect::None,
					BrightBrush->GetTint(InWidgetStyle) * FLinearColor(0.5f, 0.5f, 0.5f));
			}
		}

		if (bShowMean)
		{
			const float DrawLocationX = FMath::Max(MeanLocationMinX, SizeOfFurthur);
			if (DrawLocationX < SizeX - SizeOfFurthur && MeanLocationMaxX > SizeOfFurthur)
			{
				const float DrawSizeX = FMath::Clamp(MeanLocationMaxX - DrawLocationX, 1.f, SizeX - DrawLocationX - SizeOfFurthur);
				if (bShowMean)
				{
					FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(DrawSizeX, SizeOfBoxY), FSlateLayoutTransform(FVector2D(DrawLocationX, LocationOfMeanBoxY))),
						DarkBrush, ESlateDrawEffect::None,
						BrightBrush->GetTint(InWidgetStyle) * FLinearColor(0.2f, 0.2f, 0.2f));
				}
			}
		}

		// show the sigma
		if (bShowSigma)
		{
			const float SizeOfSigmaY = 4.f;
			const float LocationOfSigmaY = (SizeY / 2.f) - 2.f;
			{
				const float SigmaLocationMinX = MeanLocationMinX - (OldestSigma * SizePerSeconds * NumberOfSigma);
				const float SigmaLocationMaxX = MeanLocationMinX + (OldestSigma * SizePerSeconds * NumberOfSigma);
				if (SigmaLocationMaxX > SizeOfFurthur)
				{
					const float DrawLocationX = FMath::Clamp(SigmaLocationMinX, SizeOfFurthur, SizeX - SizeOfFurthur);
					const float DrawSizeX = FMath::Clamp(SigmaLocationMaxX - DrawLocationX, 0.f, SizeX - SizeOfFurthur);
					FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(DrawSizeX, SizeOfSigmaY), FSlateLayoutTransform(FVector2D(DrawLocationX, LocationOfSigmaY))),
						DarkBrush, ESlateDrawEffect::None,
						BrightBrush->GetTint(InWidgetStyle) * FLinearColor::White);
				}
			}
			{
				const float SigmaLocationMinX = MeanLocationMaxX - (NewestSigma * SizePerSeconds * NumberOfSigma);
				const float SigmaLocationMaxX = MeanLocationMaxX + (NewestSigma * SizePerSeconds * NumberOfSigma);
				if (SigmaLocationMinX < SizeX - SizeOfFurthur)
				{
					const float DrawLocationX = FMath::Clamp(SigmaLocationMinX, SizeOfFurthur, SizeX - SizeOfFurthur);
					const float DrawSizeX = FMath::Clamp(SigmaLocationMaxX - DrawLocationX, 0.f, SizeX - SizeOfFurthur);
					FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
						AllottedGeometry.ToPaintGeometry(FVector2D(DrawSizeX, SizeOfSigmaY), FSlateLayoutTransform(FVector2D(DrawLocationX, LocationOfSigmaY))),
						DarkBrush, ESlateDrawEffect::None,
						BrightBrush->GetTint(InWidgetStyle) * FLinearColor::White);
				}
			}
		}

		// square to show that the data goes further
		if (bShowFurther)
		{
			if (bShowSnapshot && SnapshotLocationMaxX > SizeX - SizeOfFurthur)
			{
				FSlateDrawElement::MakeText(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(SizeOfFurthur, SizeOfBoxY), FSlateLayoutTransform(FVector2D(SizeX - SizeOfFurthur + 4, LocationOfSnapshotBoxY))),
					FEditorFontGlyphs::Angle_Double_Right,
					FontInfo, ESlateDrawEffect::None,
					FLinearColor::White);
			}

			if (bShowMean && MeanLocationMaxX > SizeX - SizeOfFurthur)
			{
				FSlateDrawElement::MakeText(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(SizeOfFurthur, SizeOfBoxY), FSlateLayoutTransform(FVector2D(SizeX - SizeOfFurthur + 4, LocationOfSnapshotBoxY))),
					FEditorFontGlyphs::Angle_Double_Right,
					FontInfo, ESlateDrawEffect::None,
					FLinearColor::White);
			}
		}

		// Draw evaluation line
		{
			const bool bIsInRange = (FMath::IsNearlyEqual(EvaluationTime, MinSampleTime) || EvaluationTime >= MinSampleTime) && (FMath::IsNearlyEqual(EvaluationTime, MaxSampleTime) || EvaluationTime <= MaxSampleTime);
			FLinearColor EvaluationColor = bIsInRange ? FLinearColor::Green : FLinearColor::Red;
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(1, SizeY), FSlateLayoutTransform(FVector2D(LocationOfCenter, 0))),
				DarkBrush, ESlateDrawEffect::None,
				DarkBrush->GetTint(InWidgetStyle) * EvaluationColor);
		}

		// Draw frame lines
		if (bDrawFrameTimes && (SampleCount > 1))
		{
			const FLinearColor EvaluationColor = FLinearColor::Black;
			const double FrameIntervalEstimate = (MaxSampleTime - MinSampleTime) / (SampleCount - 1);
			for (int32 FrameIndex = 0; FrameIndex < SampleCount; ++FrameIndex)
			{
				double SampleTime = 0.0;
				if (bUseAccurateFrameTimes && (AllSampleTimes.Num() == SampleCount))
				{
					SampleTime = AllSampleTimes[FrameIndex];
				}
				else
				{
					SampleTime = MinSampleTime + (FrameIntervalEstimate * FrameIndex);
				}

				const double LocationToDraw = LocationOfCenter + ((SampleTime - EvaluationTime) * SizePerSeconds);

				TArray<FVector2D> LinePoints;
				LinePoints.SetNum(2);

				if (LocationToDraw >= SizeOfFurthur)
				{
					LinePoints[0] = FVector2D(LocationToDraw, 0.f);
					LinePoints[1] = FVector2D(LocationToDraw, SizeY * 0.5f);

					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(),
						LinePoints,
						ESlateDrawEffect::None,
						DarkBrush->GetTint(InWidgetStyle) * EvaluationColor,
						false
					);
				}
			}
		}

		return LayerId;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(100, 20);
	}

	const FSlateBrush* DarkBrush;
	const FSlateBrush* BrightBrush;

	bool bShowFurther = false;
	bool bShowMean = true;
	bool bShowSigma = true;
	bool bShowSnapshot = true;
	TAttribute<float> SizePerSecondsAttibute;

	double EvaluationTime = 0.0;
	double MinSampleTime = 0.0;
	double MaxSampleTime = 0.0;
	TArray<double> AllSampleTimes;
	double OldestMean = 0.0;
	double NewestMean = 0.0;
	double OldestSigma = 0.0;
	double NewestSigma = 0.0;
	int32 NumberOfSigma = 3;
	int32 SampleCount = 0;
	FSlateFontInfo FontInfo;
};


/* STimingDiagramWidget
 *****************************************************************************/
void STimingDiagramWidget::Construct(const FArguments& InArgs, bool bInIsInput)
{
	bIsInput = bInIsInput;
	ChannelIdentifier = InArgs._ChannelIdentifier;
	InputIdentifier = InArgs._InputIdentifier;

	if (!bIsInput && !InputIdentifier.IsValid())
	{
		UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
		check(TimedDataMonitorSubsystem);
		InputIdentifier = TimedDataMonitorSubsystem->GetChannelInput(ChannelIdentifier);
	}

	GraphicWidget = SNew(STimingDiagramWidgetGraphic)
		.ShowFurther(InArgs._ShowFurther)
		.ShowMean(InArgs._ShowMean)
		.ShowSigma(InArgs._ShowSigma)
		.ShowMean(InArgs._ShowSnapshot)
		.UseNiceBrush(InArgs._UseNiceBrush)
		.SizePerSeconds(InArgs._SizePerSeconds);

	UpdateCachedValue();
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			GraphicWidget.ToSharedRef()
		]
	];
	
	SetToolTip(SNew(SToolTip).Text(this, &STimingDiagramWidget::GetTooltipText));
}


void STimingDiagramWidget::UpdateCachedValue()
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);

	const ETimedDataInputEvaluationType EvaluationType = TimedDataMonitorSubsystem->GetInputEvaluationType(InputIdentifier);

	const UTimedDataMonitorEditorSettings* const EditorSettings = GetDefault<UTimedDataMonitorEditorSettings>();
	const bool bUpdateSampleTimes = EditorSettings->bDrawFrameTimesInBufferVisualization && EditorSettings->bUseAccurateFrameTimesInBufferVisualization;

	double EvaluationOffset = TimedDataMonitorSubsystem->GetInputEvaluationOffsetInSeconds(InputIdentifier);
	double MinSampleTime = 0.0;
	double MaxSampleTime = 0.0;
	double NewestMean = 0.0;

	if (bIsInput)
	{
		MinSampleTime = TimedDataMonitorSubsystem->GetInputOldestDataTime(InputIdentifier).AsSeconds(EvaluationType);
		MaxSampleTime = TimedDataMonitorSubsystem->GetInputNewestDataTime(InputIdentifier).AsSeconds(EvaluationType);
		GraphicWidget->OldestMean = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToOldestSampleMean(InputIdentifier);
		GraphicWidget->NewestMean = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToNewestSampleMean(InputIdentifier);
		GraphicWidget->OldestSigma = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToOldestSampleStandardDeviation(InputIdentifier);
		GraphicWidget->NewestSigma = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToNewestSampleStandardDeviation(InputIdentifier);
	}
	else
	{
		MinSampleTime = TimedDataMonitorSubsystem->GetChannelOldestDataTime(ChannelIdentifier).AsSeconds(EvaluationType);
		MaxSampleTime = TimedDataMonitorSubsystem->GetChannelNewestDataTime(ChannelIdentifier).AsSeconds(EvaluationType);
		GraphicWidget->OldestMean = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToOldestSampleMean(ChannelIdentifier);
		GraphicWidget->NewestMean = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToNewestSampleMean(ChannelIdentifier);
		GraphicWidget->OldestSigma = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToOldestSampleStandardDeviation(ChannelIdentifier);
		GraphicWidget->NewestSigma = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToNewestSampleStandardDeviation(ChannelIdentifier);
		GraphicWidget->SampleCount = TimedDataMonitorSubsystem->GetChannelNumberOfSamples(ChannelIdentifier);

		if (bUpdateSampleTimes)
		{
			UpdateSampleTimes(TimedDataMonitorSubsystem->GetChannelFrameDataTimes(ChannelIdentifier), EvaluationType);
		}
	}
	GraphicWidget->MinSampleTime = MinSampleTime + EvaluationOffset;
	GraphicWidget->MaxSampleTime = MaxSampleTime + EvaluationOffset;
	GraphicWidget->EvaluationTime = UTimedDataMonitorSubsystem::GetEvaluationTime(EvaluationType);
}


FText STimingDiagramWidget::GetTooltipText() const
{
	UTimedDataMonitorSubsystem* TimedDataMonitorSubsystem = GEngine->GetEngineSubsystem<UTimedDataMonitorSubsystem>();
	check(TimedDataMonitorSubsystem);
	
	if (bIsInput)
	{
		const float DistanceToNewestSampleAverage = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToNewestSampleMean(InputIdentifier);
		const float DistanceToOldestSampleAverage = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToOldestSampleMean(InputIdentifier);
		const float DistanceToNewestSampleSigma = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToNewestSampleStandardDeviation(InputIdentifier);
		const float DistanceToOldestSampleSigma = TimedDataMonitorSubsystem->GetInputEvaluationDistanceToOldestSampleStandardDeviation(InputIdentifier);

		return FText::Format(LOCTEXT("TimingDiagramTooltip_DistanceToNewest", "Distance to newest:\nMean: {0}\nSigma: {1}\n\nDistance to oldest:\nMean: {2}\nSigma: {3}")
			, DistanceToNewestSampleAverage
			, DistanceToNewestSampleSigma
			, DistanceToOldestSampleAverage
			, DistanceToOldestSampleSigma);
	}
	else
	{
		const int32 SampleCount = TimedDataMonitorSubsystem->GetChannelNumberOfSamples(ChannelIdentifier);
		const float DistanceToNewestSampleAverage = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToNewestSampleMean(ChannelIdentifier);
		const float DistanceToOldestSampleAverage = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToOldestSampleMean(ChannelIdentifier);
		const float DistanceToNewestSampleSigma = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToNewestSampleStandardDeviation(ChannelIdentifier);
		const float DistanceToOldestSampleSigma = TimedDataMonitorSubsystem->GetChannelEvaluationDistanceToOldestSampleStandardDeviation(ChannelIdentifier);
		FTimedDataInputEvaluationData LastEvalData;
		TimedDataMonitorSubsystem->GetChannelLastEvaluationDataStat(ChannelIdentifier, LastEvalData);
	
		return FText::Format(LOCTEXT("TimingDiagramTooltip_SampleCount", "Sample count: {0}\n\nDistance to newest:\nLast Value: {1}\nMean: {2}\nSigma: {3}\n\nDistance to oldest:\nLast Value: {4}\nMean: {5}\nSigma: {6}")
		, SampleCount
		, LastEvalData.DistanceToNewestSampleSeconds
		, DistanceToNewestSampleAverage
		, DistanceToNewestSampleSigma
		, LastEvalData.DistanceToOldestSampleSeconds
		, DistanceToOldestSampleAverage
		, DistanceToOldestSampleSigma);
	}
}

void STimingDiagramWidget::UpdateSampleTimes(const TArray<FTimedDataChannelSampleTime>& FrameDataTimes, ETimedDataInputEvaluationType EvaluationType)
{
	// Cache the newest time in the buffer from the last update
	double PreviousNewestTime = -1.0;
	if (GraphicWidget->AllSampleTimes.Num() > 0)
	{
		PreviousNewestTime = GraphicWidget->AllSampleTimes.Last();
	}

	// If the widget's buffer is too large (because the buffer size changed), resize it and remove the oldest frames
	if (GraphicWidget->AllSampleTimes.Num() > FrameDataTimes.Num())
	{
		const int32 Count = (GraphicWidget->AllSampleTimes.Num() - FrameDataTimes.Num());
		GraphicWidget->AllSampleTimes.RemoveAt(0, Count, EAllowShrinking::Yes);
	}
	else if (GraphicWidget->AllSampleTimes.Num() < FrameDataTimes.Num())
	{
		GraphicWidget->AllSampleTimes.Reserve(FrameDataTimes.Num());
	}

	// Add all sample times that are newer than the previous newest time
	for (int32 SampleIndex = FrameDataTimes.Num() - 1; SampleIndex >= 0; --SampleIndex)
	{
		const double SampleTimeInSeconds = FrameDataTimes[SampleIndex].AsSeconds(EvaluationType);
		if (SampleTimeInSeconds > PreviousNewestTime)
		{
			// Remove the oldest sample(s) from the widget's buffer if it full 
			if (GraphicWidget->AllSampleTimes.Num() >= FrameDataTimes.Num())
			{
				constexpr int32 Count = 1;
				GraphicWidget->AllSampleTimes.RemoveAt(0, Count, EAllowShrinking::No);
			}

			GraphicWidget->AllSampleTimes.Add(SampleTimeInSeconds);
		}
		else
		{
			break;
		}
	}

}

#undef LOCTEXT_NAMESPACE
