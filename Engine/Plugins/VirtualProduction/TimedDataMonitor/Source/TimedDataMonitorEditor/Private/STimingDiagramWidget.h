// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "TimedDataMonitorTypes.h"
#include "Widgets/SCompoundWidget.h"

enum class ETimedDataInputEvaluationType : uint8;
struct FTimedDataChannelSampleTime;

class STimingDiagramWidgetGraphic;

class STimingDiagramWidget : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;
	
public:
	SLATE_BEGIN_ARGS(STimingDiagramWidget)
			: _ShowFurther(false)
			, _ShowMean(true)
			, _ShowSigma(false)
			, _ShowSnapshot(false)
			, _UseNiceBrush(false)
			, _SizePerSeconds(100)
		{ }
		SLATE_ARGUMENT(FTimedDataMonitorChannelIdentifier, ChannelIdentifier)
		SLATE_ARGUMENT(FTimedDataMonitorInputIdentifier, InputIdentifier)
		SLATE_ARGUMENT(bool, ShowFurther)
		SLATE_ARGUMENT(bool, ShowMean)
		SLATE_ARGUMENT(bool, ShowSigma)
		SLATE_ARGUMENT(bool, ShowSnapshot)
		SLATE_ARGUMENT(bool, UseNiceBrush)
		SLATE_ATTRIBUTE(float, SizePerSeconds)
	SLATE_END_ARGS()
	
public:
	void Construct(const FArguments& InArgs, bool bIsInput);
	
	void UpdateCachedValue();

private:
	FText GetTooltipText() const;
	void UpdateSampleTimes(const TArray<FTimedDataChannelSampleTime>& FrameDataTimes, ETimedDataInputEvaluationType EvaluationType);

private:
	// Identifier of the channel this widget is associated to
	TSharedPtr<STimingDiagramWidgetGraphic> GraphicWidget;
	FTimedDataMonitorChannelIdentifier ChannelIdentifier;
	FTimedDataMonitorInputIdentifier InputIdentifier;
	bool bIsInput;
};
