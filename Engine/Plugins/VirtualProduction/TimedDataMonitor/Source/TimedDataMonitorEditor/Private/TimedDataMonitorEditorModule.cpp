// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimedDataMonitorEditorModule.h"

#include "Framework/Application/SlateApplication.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "STimedDataMonitorPanel.h"
#include "STimedDataMonitorBufferVisualizer.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"

#define LOCTEXT_NAMESPACE "TimedDataMonitorEditorModule"

namespace UE::TDM
{
	static const FName NAME_LevelEditorModuleName("LevelEditor");
}


void FTimedDataMonitorEditorModule::StartupModule()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "Timed Data Monitor",
			LOCTEXT("SettingsName", "Timed Data Monitor"),
			LOCTEXT("Description", "Configure the Timed Data Monitor panel."),
			GetMutableDefault<UTimedDataMonitorEditorSettings>()
		);
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(UE::TDM::NAME_LevelEditorModuleName);
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	auto RegisterLambda = [this]()
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(UE::TDM::NAME_LevelEditorModuleName);
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
			if (LevelEditorTabManager)
			{
				RegisterNomadTabSpawner(LevelEditorTabManager);
			}
		};

	if (LevelEditorTabManager)
	{
		RegisterLambda();
	}
	else
	{
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterLambda);
	}
}

void FTimedDataMonitorEditorModule::ShutdownModule()
{
	if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Editor", "Plugins", "Timed Data Monitor");
		}

		if (FSlateApplication::IsInitialized())
		{
			if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(UE::TDM::NAME_LevelEditorModuleName))
			{
				LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);

				if (TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager())
				{
					UnregisterNomadTabSpawner(LevelEditorTabManager);
				}
			}
		}
	}
}

void FTimedDataMonitorEditorModule::RegisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager)
{
	STimedDataMonitorPanel::RegisterNomadTabSpawner(TabManager, WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory());
}

void FTimedDataMonitorEditorModule::UnregisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager)
{
	STimedDataMonitorPanel::UnregisterNomadTabSpawner(TabManager);
}

void FTimedDataMonitorEditorModule::DisplayTimedDataMonitorPanel(TSharedPtr<FTabManager> TabManager)
{
	TabManager->TryInvokeTab(STimedDataMonitorPanel::TabName);
}

IMPLEMENT_MODULE(FTimedDataMonitorEditorModule, TimedDataMonitorEditor);

#undef LOCTEXT_NAMESPACE
