// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "TimedDataMonitorCalibration.h"

#include "TimedDataMonitorEditorSettings.generated.h"


UENUM()
enum class ETimedDataMonitorEditorCalibrationType
{
	CalibrateWithTimecode = 0,
	TimeCorrection = 1,
	Max = 2
};


UCLASS(config = EditorPerProjectUserSettings, MinimalAPI)
class UTimedDataMonitorEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Option to use when calibrating from the UI. */
	UPROPERTY(Config, EditAnywhere, Category = "Calibration")
	FTimedDataMonitorCalibrationParameters CalibrationSettings;

	/** Option to use when calibrating from the UI. */
	UPROPERTY(Config, EditAnywhere, Category = "Calibration")
	FTimedDataMonitorTimeCorrectionParameters TimeCorrectionSettings;

	/** At which speed we should update the Timed Data Monitor UI. */
	UPROPERTY(Config, EditAnywhere, Category ="UI", meta=(ClampMin=0.0f))
	float RefreshRate = 0.2f;

	/** Whether or not to draw vertical bars delineating each frame in the buffer of a channel */
	UPROPERTY(Config, EditAnywhere, Category = "UI")
	bool bDrawFrameTimesInBufferVisualization = true;

	/** If true, accurate world times will be used to draw frame times. If false, a constant frame interval is used */
	UPROPERTY(Config, EditAnywhere, Category = "UI", meta = (EditCondition = "bDrawFrameTimesInBufferVisualization"))
	bool bUseAccurateFrameTimesInBufferVisualization = false;

	UPROPERTY(Config, AdvancedDisplay, EditAnywhere, Category = "UI", meta = (InlineEditConditionToggle = true))
	bool bOverrideNumberOfStandardDeviationToShow;

	/**
	 * When displaying the buffer widget, how many STD should we show.
	 * By default, it will show the value used for Calibration or Time Correction.
	 */
	UPROPERTY(Config, AdvancedDisplay, EditAnywhere, Category = "UI", meta = (ClampMin = 0, ClampMax = 5, EditCondition = "bOverrideNumberOfStandardDeviationToShow"))
	int32 OverrideNumberOfStandardDeviationToShow = 5;

	/**
	 * The threshold at which to display the frame time as orange.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI", meta = (ClampMin = 0, ClampMax = 2))
	double FrameTimeWarnThreshold = 0.85;

	/**
	 * The percent at which to display the target frame time in the frame time indicator. 
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI", meta = (ClampMin = 0, ClampMax = 1))
	double TargetFrameTimePercent = 0.8;

	UPROPERTY(Config)
	ETimedDataMonitorEditorCalibrationType LastCalibrationType = ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode;

	/** The Reset button will reset the buffer errors count. */
	UPROPERTY(Config, EditAnywhere, Category = "UI|Reset")
	bool bResetBufferStatEnabled = true;

	/** The Reset button will clear the messages list. */
	UPROPERTY(Config, EditAnywhere, Category = "UI|Reset")
	bool bClearMessageEnabled = true;

	/** The Reset button will set the evaluation time of all input to 0. */
	UPROPERTY(Config, EditAnywhere, Category = "UI|Reset")
	bool bResetEvaluationTimeEnabled = false;

public:
	int32 GetNumberOfStandardDeviationForUI() const
	{
		if (bOverrideNumberOfStandardDeviationToShow)
		{
			return OverrideNumberOfStandardDeviationToShow;
		}
		else if (LastCalibrationType == ETimedDataMonitorEditorCalibrationType::CalibrateWithTimecode && CalibrationSettings.bUseStandardDeviation)
		{
			return CalibrationSettings.NumberOfStandardDeviation;
		}
		else if (LastCalibrationType == ETimedDataMonitorEditorCalibrationType::TimeCorrection && TimeCorrectionSettings.bUseStandardDeviation)
		{
			return TimeCorrectionSettings.NumberOfStandardDeviation;
		}
		return 0;
	}
};
