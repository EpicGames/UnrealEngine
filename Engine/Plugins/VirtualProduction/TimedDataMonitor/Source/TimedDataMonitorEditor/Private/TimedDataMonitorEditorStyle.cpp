// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimedDataMonitorEditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateColorBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

const FName FTimedDataMonitorEditorStyle::NAME_TimecodeBrush = "Img.Timecode.Small";
const FName FTimedDataMonitorEditorStyle::NAME_PlatformTimeBrush = "Img.PlatformTime.Small";
const FName FTimedDataMonitorEditorStyle::NAME_NoEvaluationBrush = "Img.NoEvaluation.Small";


FTimedDataMonitorEditorStyle::FTimedDataMonitorEditorStyle()
	: FSlateStyleSet("TimedDataSourceEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);

	FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("TimedDataMonitor"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// CheckBox
	{
		FSlateImageBrush SwitchOn = IMAGE_BRUSH(TEXT("Widgets/Switch_ON"), FVector2D(28.f, 14.f));
		FSlateImageBrush SwitchOff = IMAGE_BRUSH(TEXT("Widgets/Switch_OFF"), FVector2D(28.f, 14.f));
		FSlateImageBrush SwitchUndeterminded = IMAGE_BRUSH(TEXT("Widgets/Switch_Undetermined"), FVector2D(28.f, 14.f));

		FCheckBoxStyle SwitchStyle = FCheckBoxStyle()
			.SetForegroundColor(FLinearColor::White)
			.SetUncheckedImage(SwitchOff)
			.SetUncheckedHoveredImage(SwitchOff)
			.SetUncheckedPressedImage(SwitchOff)
			.SetUndeterminedImage(SwitchUndeterminded)
			.SetUndeterminedHoveredImage(SwitchUndeterminded)
			.SetUndeterminedPressedImage(SwitchUndeterminded)
			.SetCheckedImage(SwitchOn)
			.SetCheckedHoveredImage(SwitchOn)
			.SetCheckedPressedImage(SwitchOn)
			.SetPadding(FMargin(0, 0, 0, 1));
		Set("CheckBox.Enable", SwitchStyle);
	}

	// brush
	{
		Set("Brush.White", new FSlateColorBrush(FLinearColor::White));
	}

	// images
	{
		const FVector2D Icon20x20(20.0f, 20.0f);
		Set("Img.TimedDataMonitor.Icon", new IMAGE_BRUSH_SVG("Common/TimedDataMonitor", Icon20x20));

		Set(NAME_TimecodeBrush, new IMAGE_BRUSH(TEXT("Widgets/Timecode_16x"), Icon16x16));
		Set(NAME_PlatformTimeBrush, new IMAGE_BRUSH(TEXT("Widgets/Time_16x"), Icon16x16));
		Set(NAME_NoEvaluationBrush, new IMAGE_BRUSH(TEXT("Widgets/NoEvaluation_16x"), Icon16x16));

		Set("Img.BufferVisualization", new IMAGE_BRUSH(TEXT("Widgets/BufferVisualization_24x"), Icon24x24));
		Set("Img.Calibration", new IMAGE_BRUSH(TEXT("Widgets/Calibration_24x"), Icon24x24));
		Set("Img.TimeCorrection", new IMAGE_BRUSH(TEXT("Widgets/TimeCorrection_24x"), Icon24x24));
		Set("Img.Edit", new IMAGE_BRUSH(TEXT("Widgets/Edit_24x"), Icon24x24));
		
	}

	// font
	{
		Set("Font.Regular", FCoreStyle::GetDefaultFontStyle("Regular", FCoreStyle::RegularTextSize));
		Set("Font.Large", FCoreStyle::GetDefaultFontStyle("Regular", 12));
	}

	// text block
	{
		FTextBlockStyle NormalText = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		Set("TextBlock.Regular", FTextBlockStyle(NormalText).SetFont(GetFontStyle("Font.Regular")));
		Set("TextBlock.Large", FTextBlockStyle(NormalText).SetFont(GetFontStyle("Font.Large")));
	}

	// TableView
	{
		const FTableRowStyle& DefaultTableRow = FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
		Set("TableView.Child", FTableRowStyle(DefaultTableRow)
			.SetEvenRowBackgroundHoveredBrush(DefaultTableRow.InactiveBrush)
			.SetOddRowBackgroundHoveredBrush(DefaultTableRow.InactiveBrush)
			.SetActiveHoveredBrush(DefaultTableRow.InactiveBrush)
			.SetInactiveHoveredBrush(DefaultTableRow.InactiveBrush)
			.SetEvenRowBackgroundBrush(DefaultTableRow.InactiveBrush)
			.SetOddRowBackgroundBrush(DefaultTableRow.InactiveBrush)
			.SetActiveBrush(DefaultTableRow.InactiveBrush)
			.SetInactiveBrush(DefaultTableRow.InactiveBrush)
		);
	}

	// ComboButton
	{
		FComboButtonStyle SectionComboButton = FComboButtonStyle()
			.SetButtonStyle(
				FButtonStyle()
				.SetNormal(FSlateNoResource())
				.SetHovered(FSlateNoResource())
				.SetPressed(FSlateNoResource())
				.SetNormalPadding(FMargin(0, 0, 0, 0))
				.SetPressedPadding(FMargin(0, 1, 0, 0))
			)
			.SetDownArrowImage(FSlateNoResource())
			.SetMenuBorderBrush(FSlateNoResource());
		SectionComboButton.UnlinkColors();
		Set("FlatComboButton", SectionComboButton);

		const FCheckBoxStyle& ToggleButtonStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		FComboButtonStyle ToggleComboButton = FComboButtonStyle()
			.SetButtonStyle(
				FButtonStyle()
				.SetNormal(ToggleButtonStyle.UncheckedImage)
				.SetHovered(ToggleButtonStyle.UncheckedHoveredImage)
				.SetPressed(ToggleButtonStyle.UncheckedPressedImage)
				.SetNormalPadding(FMargin(0, 0, 0, 0))
				.SetPressedPadding(FMargin(0, 1, 0, 0))
			)
			.SetDownArrowImage(FSlateNoResource())
			.SetMenuBorderBrush(FSlateNoResource());
		Set("ToggleComboButton", ToggleComboButton);
	}

	// Button
	{
		FButtonStyle FlatButton = FButtonStyle()
			.SetNormal(BOX_BRUSH(TEXT("Common/ButtonHoverHint"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.15f)))
			.SetHovered(BOX_BRUSH(TEXT("Common/ButtonHoverHint"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.25f)))
			.SetPressed(BOX_BRUSH(TEXT("Common/ButtonHoverHint"), FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 0.30f)))
			.SetNormalPadding(FMargin(0.f, 2.f))
			.SetPressedPadding(FMargin(0, 1, 0, 0));
		Set("FlatButton", FlatButton);

		const FCheckBoxStyle& ToggleButtonStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		FButtonStyle ToggleButton = FButtonStyle()
			.SetNormal(ToggleButtonStyle.UncheckedImage)
			.SetHovered(ToggleButtonStyle.UncheckedHoveredImage)
			.SetPressed(ToggleButtonStyle.UncheckedPressedImage)
			.SetNormalPadding(FMargin(0, 0, 0, 1))
			.SetPressedPadding(FMargin(0, 1, 0, 0));
		Set("ToggleButton", ToggleButton);


		const FVector2D Icon12x12(12.0f, 12.0f);
		Set("PlusButton", new FSlateImageBrush( FAppStyle::Get().GetContentRootDir() / (TEXT("Icons/PlusSymbol_12x.png")), Icon12x12, FLinearColor::Gray));
		Set("MinusButton", new FSlateImageBrush( FAppStyle::Get().GetContentRootDir() / TEXT("Icons/MinusSymbol_12x.png"), Icon12x12,  FLinearColor::Gray));
	}
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FTimedDataMonitorEditorStyle::~FTimedDataMonitorEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FTimedDataMonitorEditorStyle& FTimedDataMonitorEditorStyle::Get()
{
	static FTimedDataMonitorEditorStyle Inst;
	return Inst;
}


