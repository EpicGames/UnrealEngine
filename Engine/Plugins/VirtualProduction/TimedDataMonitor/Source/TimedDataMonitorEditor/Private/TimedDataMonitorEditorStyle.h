// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * Slate style set that defines all the styles for the monitor UI
 */
class TIMEDDATAMONITOREDITOR_API FTimedDataMonitorEditorStyle : public FSlateStyleSet
{
public:
	/** Access the singleton instance for this style set */
	static FTimedDataMonitorEditorStyle& Get();

	static const FName NAME_TimecodeBrush;
	static const FName NAME_PlatformTimeBrush;
	static const FName NAME_NoEvaluationBrush;

private:
	FTimedDataMonitorEditorStyle();
	~FTimedDataMonitorEditorStyle();
};
