// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class FTabManager;
class SWidget;

class FTimedDataMonitorEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Register a tab spawner for the TimedDataMonitorPanel. */
	TIMEDDATAMONITOREDITOR_API void RegisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager);
	/** Unregister a tab spawner for the TimedDataMonitorPanel. */
	TIMEDDATAMONITOREDITOR_API void UnregisterNomadTabSpawner(TSharedPtr<FTabManager> TabManager);
	/** Try invoking the TimedDataMonitor tab. */
	TIMEDDATAMONITOREDITOR_API void DisplayTimedDataMonitorPanel(TSharedPtr<FTabManager> TabManager);
private:
	/** Handle to the delegate used to register the tab spawner when the level editor tab manager changes. */
	FDelegateHandle LevelEditorTabManagerChangedHandle;
};

