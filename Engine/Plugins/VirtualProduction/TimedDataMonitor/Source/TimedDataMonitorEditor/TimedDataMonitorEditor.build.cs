// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TimedDataMonitorEditor : ModuleRules
{
	public TimedDataMonitorEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"AssetTools",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"EditorWidgets",
				"Engine",
				"EditorStyle",
				"InputCore",
				"LiveLink",
				"LiveLinkInterface",
				"MessageLog",
				"Projects",
				"Settings",
				"Slate",
				"SlateCore",
				"TimedDataMonitor",
				"TimeManagement",
				"UnrealEd",
				"WorkspaceMenuStructure",
			});
	}
}
